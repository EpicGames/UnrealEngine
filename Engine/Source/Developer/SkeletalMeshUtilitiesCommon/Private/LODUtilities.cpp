// Copyright Epic Games, Inc. All Rights Reserved.

#include "LODUtilities.h"

#if WITH_EDITOR

#include "Algo/Accumulate.h"
#include "Animation/MorphTarget.h"
#include "Rendering/SkinWeightProfile.h"
#include "Async/ParallelFor.h"
#include "Async/TaskGraphInterfaces.h"
#include "BoneWeights.h"
#include "ClothingAsset.h"
#include "ComponentReregisterContext.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "EditorFramework/AssetImportData.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkeletalMeshLODSettings.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Engine/SkinnedAssetAsyncCompileUtils.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/Texture2D.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GenericQuadTree.h"
#include "IMeshReductionManagerModule.h"
#include "ImageCore.h"
#include "ImageCoreUtils.h"
#include "ImportUtils/SkelImport.h"
#include "Interfaces/ITargetPlatform.h"
#include "Logging/StructuredLog.h"
#include "MeshUtilities.h"
#include "MeshUtilitiesCommon.h"
#include "Misc/FeedbackContext.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "ObjectTools.h"
#include "OverlappingCorners.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshModel.h"
#include "SkeletalMeshAttributes.h"
#include "Tasks/Task.h"
#include "UObject/GarbageCollection.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Notifications/SNotificationList.h"

#include <limits>

IMPLEMENT_MODULE(FDefaultModuleImpl, SkeletalMeshUtilitiesCommon)

#define LOCTEXT_NAMESPACE "LODUtilities"

DEFINE_LOG_CATEGORY_STATIC(LogLODUtilities, Log, All);

/**
* Process and update the vertex Influences using the predefined wedges
*
* @param VertexCount - The number of vertices in the corresponding mesh.
* @param Influences - BoneWeights and Ids for the corresponding vertices.
*/
void FLODUtilities::ProcessImportMeshInfluences(const int32 VertexCount, TArray<SkeletalMeshImportData::FRawBoneInfluence>& Influences, const FString& MeshName)
{

	// Sort influences by vertex index.
	struct FCompareVertexIndex
	{
		bool operator()(const SkeletalMeshImportData::FRawBoneInfluence& A, const SkeletalMeshImportData::FRawBoneInfluence& B) const
		{
			if (A.VertexIndex > B.VertexIndex) return false;
			else if (A.VertexIndex < B.VertexIndex) return true;
			else if (A.Weight < B.Weight) return false;
			else if (A.Weight > B.Weight) return true;
			else if (A.BoneIndex > B.BoneIndex) return false;
			else if (A.BoneIndex < B.BoneIndex) return true;
			else									  return  false;
		}
	};
	Influences.Sort(FCompareVertexIndex());

	TArray <SkeletalMeshImportData::FRawBoneInfluence> NewInfluences;
	int32	LastNewInfluenceIndex = 0;
	int32	LastVertexIndex = INDEX_NONE;
	int32	InfluenceCount = 0;

	float TotalWeight = 0.f;

	int MaxVertexInfluence = 0;
	float MaxIgnoredWeight = 0.0f;

	//We have to normalize the data before filtering influences
	//Because influence filtering is base on the normalize value.
	//Some DCC like Daz studio don't have normalized weight
	for (int32 i = 0; i < Influences.Num(); i++)
	{
		// if less than min weight, or it's more than 8, then we clear it to use weight
		InfluenceCount++;
		TotalWeight += Influences[i].Weight;
		// we have all influence for the same vertex, normalize it now
		if (i + 1 >= Influences.Num() || Influences[i].VertexIndex != Influences[i + 1].VertexIndex)
		{
			// Normalize the last set of influences.
			if (InfluenceCount && (TotalWeight != 1.0f))
			{
				float OneOverTotalWeight = 1.f / TotalWeight;
				for (int r = 0; r < InfluenceCount; r++)
				{
					Influences[i - r].Weight *= OneOverTotalWeight;
				}
			}

			if (MaxVertexInfluence < InfluenceCount)
			{
				MaxVertexInfluence = InfluenceCount;
			}

			// clear to count next one
			InfluenceCount = 0;
			TotalWeight = 0.f;
		}

		if (InfluenceCount > MAX_TOTAL_INFLUENCES &&  Influences[i].Weight > MaxIgnoredWeight)
		{
			MaxIgnoredWeight = Influences[i].Weight;
		}
	}

	// warn about too many influences
	if (MaxVertexInfluence > MAX_TOTAL_INFLUENCES)
	{
		UE_LOGF(LogLODUtilities, Display, "Skeletal mesh (%ls) influence count of %d exceeds max count of %d. Influence truncation will occur. Maximum Ignored Weight %f", *MeshName, MaxVertexInfluence, MAX_TOTAL_INFLUENCES, MaxIgnoredWeight);
	}

	for (int32 i = 0; i < Influences.Num(); i++)
	{
		// we found next verts, normalize it now
		if (LastVertexIndex != Influences[i].VertexIndex)
		{
			// Normalize the last set of influences.
			if (InfluenceCount && (TotalWeight != 1.0f))
			{
				float OneOverTotalWeight = 1.f / TotalWeight;
				for (int r = 0; r < InfluenceCount; r++)
				{
					NewInfluences[LastNewInfluenceIndex - r].Weight *= OneOverTotalWeight;
				}
			}

			// now we insert missing verts
			if (LastVertexIndex != INDEX_NONE)
			{
				int32 CurrentVertexIndex = Influences[i].VertexIndex;
				for (int32 j = LastVertexIndex + 1; j < CurrentVertexIndex; j++)
				{
					// Add a 0-bone weight if none other present (known to happen with certain MAX skeletal setups).
					LastNewInfluenceIndex = NewInfluences.AddUninitialized();
					NewInfluences[LastNewInfluenceIndex].VertexIndex = j;
					NewInfluences[LastNewInfluenceIndex].BoneIndex = 0;
					NewInfluences[LastNewInfluenceIndex].Weight = 1.f;
				}
			}

			// clear to count next one
			InfluenceCount = 0;
			TotalWeight = 0.f;
			LastVertexIndex = Influences[i].VertexIndex;
		}

		// if less than min weight, or it's more than 8, then we clear it to use weight
		if (Influences[i].Weight >= UE::AnimationCore::BoneWeightThreshold &&
			InfluenceCount < MAX_TOTAL_INFLUENCES)
		{
			LastNewInfluenceIndex = NewInfluences.Add(Influences[i]);
			InfluenceCount++;
			TotalWeight += Influences[i].Weight;
		}
	}

	Influences = NewInfluences;

	// Ensure that each vertex has at least one influence as e.g. CreateSkinningStream relies on it.
	// The below code relies on influences being sorted by vertex index.
	if (Influences.Num() == 0)
	{
		// warn about no influences
		UE_LOGF(LogLODUtilities, Display, "Warning skeletal mesh (%ls) has no vertex influences", *MeshName);
		// add one for each wedge entry
		Influences.AddUninitialized(VertexCount);
		for (int32 VertexIdx = 0; VertexIdx < VertexCount; VertexIdx++)
		{
			Influences[VertexIdx].VertexIndex = VertexIdx;
			Influences[VertexIdx].BoneIndex = 0;
			Influences[VertexIdx].Weight = 1.0f;
		}
		for (int32 i = 0; i < Influences.Num(); i++)
		{
			int32 CurrentVertexIndex = Influences[i].VertexIndex;

			if (LastVertexIndex != CurrentVertexIndex)
			{
				for (int32 j = LastVertexIndex + 1; j < CurrentVertexIndex; j++)
				{
					// Add a 0-bone weight if none other present (known to happen with certain MAX skeletal setups).
					Influences.InsertUninitialized(i, 1);
					Influences[i].VertexIndex = j;
					Influences[i].BoneIndex = 0;
					Influences[i].Weight = 1.f;
				}
				LastVertexIndex = CurrentVertexIndex;
			}
		}
	}
}


bool FLODUtilities::RegenerateLOD(USkeletalMesh* SkeletalMesh, const ITargetPlatform* TargetPlatform, int32 NewLODCount /*= 0*/, bool bRegenerateEvenIfImported /*= false*/, bool bGenerateBaseLOD /*= false*/)
{
	if (SkeletalMesh)
	{
		FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkeletalMesh);

		// Unbind any existing clothing assets before we regenerate all LODs
		TArray<ClothingAssetUtils::FClothingAssetMeshBinding> ClothingBindings;
		FLODUtilities::UnbindClothingAndBackup(SkeletalMesh, ClothingBindings);

		int32 LODCount = SkeletalMesh->GetLODNum();

		if (NewLODCount > 0)
		{
			LODCount = NewLODCount;
		}

		SkeletalMesh->Modify();

		FSkeletalMeshUpdateContext UpdateContext;
		UpdateContext.SkeletalMesh = SkeletalMesh;

		//If we force a regenerate, we want to invalidate the DCC so the render data get rebuilded
		SkeletalMesh->InvalidateDeriveDataCacheGUID();

		// remove LODs
		int32 CurrentNumLODs = SkeletalMesh->GetLODNum();
		if (LODCount < CurrentNumLODs)
		{
			for (int32 LODIdx = CurrentNumLODs - 1; LODIdx >= LODCount; LODIdx--)
			{
				FLODUtilities::RemoveLOD(UpdateContext, LODIdx);
			}
		}
		// we need to add more
		else if (LODCount > CurrentNumLODs)
		{
			// Only create new skeletal mesh LOD level entries, we cannot multi thread since the LOD will be create here
			//TArray are not thread safe.
			for (int32 LODIdx = CurrentNumLODs; LODIdx < LODCount; LODIdx++)
			{
				// if no previous setting found, it will use default setting. 
				FLODUtilities::SimplifySkeletalMeshLOD(UpdateContext, LODIdx, TargetPlatform, false);
			}
		}
		else
		{
			for (int32 LODIdx = 0; LODIdx < LODCount; LODIdx++)
			{
				FSkeletalMeshLODInfo& CurrentLODInfo = *(SkeletalMesh->GetLODInfo(LODIdx));
				if ((bRegenerateEvenIfImported && LODIdx > 0) || (bGenerateBaseLOD && LODIdx == 0) || CurrentLODInfo.bHasBeenSimplified )
				{
					FLODUtilities::SimplifySkeletalMeshLOD(UpdateContext, LODIdx, TargetPlatform, false);
				}
			}
		}

		//Restore all clothing we can
		FLODUtilities::RestoreClothingFromBackup(SkeletalMesh, ClothingBindings);

		return true;
	}

	return false;
}

namespace RemoveLODHelper
{
	void GetDependentLODs(USkeletalMesh* SkeletalMesh, const int32 RefLODIndex, TArray<int32>& DependentLODs)
	{
		if (!SkeletalMesh || RefLODIndex >= SkeletalMesh->GetLODNum()-1)
		{
			return;
		}
		int32 LODCount = SkeletalMesh->GetLODNum();
		FSkeletalMeshModel* SkelMeshModel = SkeletalMesh->GetImportedModel();
		for (int32 LODIndex = RefLODIndex + 1; LODIndex < LODCount; ++LODIndex)
		{
			if (!SkeletalMesh->IsReductionActive(LODIndex))
			{
				continue;
			}
			const FSkeletalMeshLODInfo* LODInfo = SkeletalMesh->GetLODInfo(LODIndex);
			if (!LODInfo)
			{
				continue;
			}
			if (LODInfo->ReductionSettings.BaseLOD == RefLODIndex)
			{
				DependentLODs.Add(LODIndex);
			}
		}
	}

	void AdjustReductionSettings(USkeletalMesh* SkeletalMesh, const int32 DestinationLODIndex, const int32 SourceLODIndex)
	{
		FSkeletalMeshLODInfo* DestinationLODInfo = SkeletalMesh->GetLODInfo(DestinationLODIndex);
		const FSkeletalMeshLODInfo* SourceLODInfo = SkeletalMesh->GetLODInfo(SourceLODIndex);
		if (!DestinationLODInfo || !SourceLODInfo)
		{
			return;
		}
		//Adjust percent so we end up with the same amount.
		DestinationLODInfo->ReductionSettings.NumOfTrianglesPercentage /= SourceLODInfo->ReductionSettings.NumOfTrianglesPercentage;
		DestinationLODInfo->ReductionSettings.NumOfVertPercentage /= SourceLODInfo->ReductionSettings.NumOfVertPercentage;
	}
} //End namspace RemoveLODHelper

void FLODUtilities::RemoveLOD(FSkeletalMeshUpdateContext& UpdateContext, int32 DesiredLOD )
{
	USkeletalMesh* SkeletalMesh = UpdateContext.SkeletalMesh;
	FSkeletalMeshModel* SkelMeshModel = SkeletalMesh->GetImportedModel();

	if(SkelMeshModel->LODModels.Num() <= 1)
	{
		if(!FApp::IsUnattended())
		{
			FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "NoLODToRemove", "No LODs to remove!") );
		}
		UE_LOGF(LogLODUtilities, Warning, "Cannot remove LOD %d, there must be at least one LOD after the removal.", DesiredLOD);
		return;
	}

	check( SkeletalMesh->GetLODNum() == SkelMeshModel->LODModels.Num() );

	// If its a valid LOD, remove it.
	if(DesiredLOD < SkelMeshModel->LODModels.Num() )
	{
		FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkeletalMesh);

		//Get the dependent generated LODs
		TArray<int32> DependentLODs;
		RemoveLODHelper::GetDependentLODs(SkeletalMesh, DesiredLOD, DependentLODs);

		//Adjust LODInfo properties to be in sync with the LOD removal. We reverse iterate because we want to restore some LOD info property from the previous LOD
		for (int32 NextLODIndex = SkeletalMesh->GetLODNum() -1; NextLODIndex > DesiredLOD; NextLODIndex--)
		{
			const FSkeletalMeshLODInfo* PreviousLODInfo = SkeletalMesh->GetLODInfo(NextLODIndex-1);
			FSkeletalMeshLODInfo* NextLODInfo = SkeletalMesh->GetLODInfo(NextLODIndex);
			if (!NextLODInfo)
			{
				continue;
			}

			//Adjust the reduction baseLOD
			if(SkeletalMesh->IsReductionActive(NextLODIndex) && NextLODInfo->ReductionSettings.BaseLOD > DesiredLOD)
			{
				NextLODInfo->ReductionSettings.BaseLOD--;
			}
			
			//Propagate someproperties we need to take from the previous LOD
			if (PreviousLODInfo)
			{
				//Screen size
				NextLODInfo->ScreenSize = PreviousLODInfo->ScreenSize;
			}
		}

		//Adjust the imported data so it point on the correct LOD index
		if (DependentLODs.Num() > 0 && SkeletalMesh->HasMeshDescription(DesiredLOD))
		{
			const int32 FirstDepLODIndex = DependentLODs[0];
			if (FSkeletalMeshLODInfo* FirstDepLODInfo = SkeletalMesh->GetLODInfo(FirstDepLODIndex))
			{
				SkeletalMesh->ModifyMeshDescription(DesiredLOD);
				FMeshDescription* SourceMeshDescription = SkeletalMesh->GetMeshDescription(DesiredLOD);
				
				SkeletalMesh->ModifyMeshDescription(FirstDepLODIndex);
				SkeletalMesh->CreateMeshDescription(FirstDepLODIndex, MoveTemp(*SourceMeshDescription));
				SkeletalMesh->CommitMeshDescription(FirstDepLODIndex);

				//Manage the override original reduction source mesh data
				if (SkelMeshModel->InlineReductionCacheDatas.IsValidIndex(FirstDepLODIndex))
				{
					//The inline reduction cache data will be recache by the build
					SkelMeshModel->InlineReductionCacheDatas[FirstDepLODIndex].SetCacheGeometryInfo(MAX_uint32, MAX_uint32);
				}

				//Adjust Reduction settings
				FirstDepLODInfo->ReductionSettings.BaseLOD = FirstDepLODIndex - 1;
				RemoveLODHelper::AdjustReductionSettings(SkeletalMesh, FirstDepLODIndex, DesiredLOD);
				//Do the adjustment for the other dependent LODs
				for (int32 DependentLODsIndex = 1; DependentLODsIndex < DependentLODs.Num(); ++DependentLODsIndex)
				{
					const int32 DepLODIndex = DependentLODs[DependentLODsIndex];
					FSkeletalMeshLODInfo* DepLODInfo = SkeletalMesh->GetLODInfo(DepLODIndex);
					if (!DepLODInfo)
					{
						continue;
					}
					//Adjust Reduction settings
					DepLODInfo->ReductionSettings.BaseLOD = FirstDepLODIndex - 1;
					RemoveLODHelper::AdjustReductionSettings(SkeletalMesh, DepLODIndex, FirstDepLODIndex);
				}
			}
		}

		//remove all Morph target data for this LOD
		for (UMorphTarget* MorphTarget : SkeletalMesh->GetMorphTargets())
		{
			if (MorphTarget->HasDataForLOD(DesiredLOD))
			{
				MorphTarget->GetMorphLODModels().RemoveAt(DesiredLOD);
			}
		}

		SkelMeshModel->LODModels.RemoveAt(DesiredLOD);
		SkeletalMesh->RemoveSourceModel(DesiredLOD);
		RefreshLODChange(SkeletalMesh);

		// Adjust the force LOD to point on the same one, if we are forcing a LOD greater then the one we delete, we want to continue pointing on it
		// If we delete the LOD we are loking at, we fall back on auto LOD
		for(auto Iter = UpdateContext.AssociatedComponents.CreateIterator(); Iter; ++Iter)
		{
			USkinnedMeshComponent* SkinnedComponent = Cast<USkinnedMeshComponent>(*Iter);
			if(SkinnedComponent)
			{
				int32 CurrentForceLOD = SkinnedComponent->GetForcedLOD();
				CurrentForceLOD = CurrentForceLOD == 0 ? 0 : CurrentForceLOD-1;
				if(CurrentForceLOD == DesiredLOD)
				{
					SkinnedComponent->SetForcedLOD(0);
				}
				else if (CurrentForceLOD > DesiredLOD)
				{
					//Set back the force LOD, CurrentForceLOD was reduce by one so we simply set it.
					SkinnedComponent->SetForcedLOD(CurrentForceLOD);
				}
			}
		}

		//Notify calling system of change
		UpdateContext.OnLODChanged.ExecuteIfBound();

		// Mark things for saving.
		SkeletalMesh->MarkPackageDirty();
	}
}

void FLODUtilities::RemoveLODs(FSkeletalMeshUpdateContext& UpdateContext, const TArray<int32>& DesiredLODs)
{
	USkeletalMesh* SkeletalMesh = UpdateContext.SkeletalMesh;
	FSkeletalMeshModel* SkelMeshModel = SkeletalMesh->GetImportedModel();

	auto NoLODToRemoveDialog = []()
	{
		if (!FApp::IsUnattended())
		{
			FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "NoLODToRemove", "No LODs to remove!"));
		}
		UE_LOGF(LogLODUtilities, Warning, "No LOD to remove or there must be at least one LOD after we remove this one.");
		
	};

	if (SkelMeshModel->LODModels.Num() <= 1 || DesiredLODs.Num() < 1)
	{
		NoLODToRemoveDialog();
		return;
	}

	check(SkeletalMesh->GetLODNum() == SkelMeshModel->LODModels.Num());

	TArray<int32> SortedDesiredLODs;
	for (int32 DesiredLODIndex = 0; DesiredLODIndex < DesiredLODs.Num(); ++DesiredLODIndex)
	{
		int32 DesiredLOD = DesiredLODs[DesiredLODIndex];
		if (SkelMeshModel->LODModels.Num() > 1 && SkelMeshModel->LODModels.IsValidIndex(DesiredLOD))
		{
			SortedDesiredLODs.Add(DesiredLOD);
		}
		else
		{
			UE_LOGF(LogLODUtilities, Warning, "Cannot remove LOD %d", DesiredLOD);
		}
	}

	if (SortedDesiredLODs.Num() < 1)
	{
		NoLODToRemoveDialog();
		return;
	}

	{
		FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkeletalMesh);
		//Sort the LODs and reverse iterate the sorted array to remove the LODs from the end to avoid having to remap LODs index in the sortedDesiredLODs array
		SortedDesiredLODs.Sort();
		for (int32 SortedDesiredLODIndex = SortedDesiredLODs.Num()-1; SortedDesiredLODIndex >= 0 ; SortedDesiredLODIndex--)
		{
			int32 LODToRemove = SortedDesiredLODs[SortedDesiredLODIndex];
			check(SkelMeshModel->LODModels.IsValidIndex(LODToRemove))
			FLODUtilities::RemoveLOD(UpdateContext, LODToRemove);
		}
	}
}

namespace UE::Private
{
	struct FScopedLockProperties
	{
	public:
		FScopedLockProperties(USkeletalMesh* SkeletalMesh)
		{
			check(SkeletalMesh);
			Lock = SkeletalMesh->LockPropertiesUntil();
		}

		~FScopedLockProperties()
		{
			Lock->Trigger();
			Lock = nullptr;
		}
	private:
		FEvent* Lock = nullptr;
	};
}

bool FLODUtilities::SetCustomLOD(USkeletalMesh* DestinationSkeletalMesh, USkeletalMesh* SourceSkeletalMesh, const int32 LodIndex, const FString& SourceDataFilename)
{
	if(!DestinationSkeletalMesh || !SourceSkeletalMesh)
	{
		return false;
	}

	FScopedSkeletalMeshPostEditChange ScopePostEditChange(DestinationSkeletalMesh);
	//Lock the skeletal mesh
	UE::Private::FScopedLockProperties ScopedLock(DestinationSkeletalMesh);
	FSkinnedAssetAsyncBuildScope AsyncBuildScope(DestinationSkeletalMesh);

	// Get a list of all the clothing assets affecting this LOD so we can re-apply later
	TArray<ClothingAssetUtils::FClothingAssetMeshBinding> ClothingBindings;
	TArray<UClothingAssetBase*> ClothingAssetsInUse;
	TArray<int32> ClothingAssetSectionIndices;
	TArray<int32> ClothingAssetInternalLodIndices;

	//SAve custom imported morph targets
	TMap<FString, TArray<FMorphTargetLodBackupData>> BackupImportedMorphTargetData;

	TArray<FSkinWeightProfileInfo> ExistingSkinWeightProfileInfos;
	TArray<FSkeletalMeshImportData> ExistingAlternateImportDataPerLOD;

	FSkeletalMeshModel* const SourceImportedResource = SourceSkeletalMesh->GetImportedModel();
	FSkeletalMeshModel* const DestImportedResource = DestinationSkeletalMesh->GetImportedModel();

	if (!SourceImportedResource || !DestImportedResource)
	{
		return false;
	}

	if (DestImportedResource->LODModels.IsValidIndex(LodIndex))
	{
		FLODUtilities::UnbindClothingAndBackup(DestinationSkeletalMesh, ClothingBindings, LodIndex);

		//Backup the lod custom imported morph
		BackupCustomImportedMorphTargetData(DestinationSkeletalMesh, BackupImportedMorphTargetData);

		int32 ExistingLodCount = DestinationSkeletalMesh->GetLODNum();

		//Extract all LOD Skin weight profiles data, we will re-apply them at the end
		ExistingSkinWeightProfileInfos = DestinationSkeletalMesh->GetSkinWeightProfiles();
		DestinationSkeletalMesh->GetSkinWeightProfiles().Reset();
		for (int32 AllLodIndex = 0; AllLodIndex < ExistingLodCount; ++AllLodIndex)
		{
			FSkeletalMeshLODModel& BuildLODModel = DestinationSkeletalMesh->GetImportedModel()->LODModels[AllLodIndex];
			BuildLODModel.SkinWeightProfiles.Reset();

			//Store the LOD alternate skinning profile data
			FSkeletalMeshImportData& SkeletalMeshImportData = ExistingAlternateImportDataPerLOD.AddDefaulted_GetRef();
			if (DestinationSkeletalMesh->HasMeshDescription(AllLodIndex))
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				DestinationSkeletalMesh->LoadLODImportedData(AllLodIndex, SkeletalMeshImportData);
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}
	}

	//Lambda to call to re-apply the clothing
	auto ReapplyClothing = [&DestinationSkeletalMesh, &ClothingBindings, &SourceImportedResource, &LodIndex]()
	{
		if (SourceImportedResource->LODModels.IsValidIndex(LodIndex))
		{
			// Re-apply our clothing assets
			FLODUtilities::RestoreClothingFromBackup(DestinationSkeletalMesh, ClothingBindings, LodIndex);
		}
	};

	auto ReapplyCustomImportedMorphTarget = [&DestinationSkeletalMesh, &BackupImportedMorphTargetData, &LodIndex]()
	{
		if (FMeshDescription* MeshDescription = DestinationSkeletalMesh->GetMeshDescription(LodIndex))
		{
			if (FLODUtilities::RestoreCustomImportedMorphTargetData(DestinationSkeletalMesh, LodIndex, *MeshDescription, BackupImportedMorphTargetData))
			{
				USkeletalMesh::FCommitMeshDescriptionParams CommitParams;
				CommitParams.bForceUpdate = false;
				CommitParams.bMarkPackageDirty = false;
				DestinationSkeletalMesh->CommitMeshDescription(LodIndex, CommitParams);
			}
		}
	};

	auto ReapplyAlternateSkinning = [&DestinationSkeletalMesh, &ExistingSkinWeightProfileInfos, &ExistingAlternateImportDataPerLOD, &LodIndex]()
	{
		if (ExistingSkinWeightProfileInfos.Num() > 0)
		{
			TArray<FSkinWeightProfileInfo>& SkinProfiles = DestinationSkeletalMesh->GetSkinWeightProfiles();
			SkinProfiles = ExistingSkinWeightProfileInfos;
			for (const FSkinWeightProfileInfo& ProfileInfo : SkinProfiles)
			{
				const int32 LodCount = DestinationSkeletalMesh->GetLODNum();
				for(int32 AllLodIndex = 0; AllLodIndex < LodCount; ++AllLodIndex)
				{
					if (!ExistingAlternateImportDataPerLOD.IsValidIndex(AllLodIndex))
					{
						continue;
					}
					if (!DestinationSkeletalMesh->HasMeshDescription(AllLodIndex))
					{
						continue;
					}
					const FSkeletalMeshLODInfo* LodInfo = DestinationSkeletalMesh->GetLODInfo(AllLodIndex);
					if (!LodInfo)
					{
						continue;
					}

					const FSkeletalMeshImportData& ExistingImportDataSrc = ExistingAlternateImportDataPerLOD[AllLodIndex];

					const FString ProfileNameStr = ProfileInfo.Name.ToString();

					FSkeletalMeshImportData ImportDataDest;
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					DestinationSkeletalMesh->LoadLODImportedData(AllLodIndex, ImportDataDest);
					PRAGMA_ENABLE_DEPRECATION_WARNINGS

					int32 PointNumberDest = ImportDataDest.Points.Num();
					int32 VertexNumberDest = ImportDataDest.Points.Num();

					if (ExistingImportDataSrc.Points.Num() != PointNumberDest)
					{
						UE_LOGF(LogLODUtilities, Error, "Alternate skinning mesh vertex number is different from the mesh LOD, we cannot apply the existing alternate skinning [%ls] on the re-import skeletal mesh LOD %d [%ls]", *ProfileNameStr, LodIndex, *DestinationSkeletalMesh->GetName());
						continue;
					}

					//Replace the data into the destination bulk data and save it
					int32 ProfileIndex = 0;
					if (ImportDataDest.AlternateInfluenceProfileNames.Find(ProfileNameStr, ProfileIndex))
					{
						ImportDataDest.AlternateInfluenceProfileNames.RemoveAt(ProfileIndex);
						ImportDataDest.AlternateInfluences.RemoveAt(ProfileIndex);
					}
					int32 SrcProfileIndex = 0;
					if (ExistingImportDataSrc.AlternateInfluenceProfileNames.Find(ProfileNameStr, SrcProfileIndex))
					{
						ImportDataDest.AlternateInfluenceProfileNames.Add(ProfileNameStr);
						ImportDataDest.AlternateInfluences.Add(ExistingImportDataSrc.AlternateInfluences[SrcProfileIndex]);
					}

					//Resave the bulk data with the new or refreshed data
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					DestinationSkeletalMesh->SaveLODImportedData(AllLodIndex, ImportDataDest);
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
			}
		}
	};

	// Now we copy the base FSkeletalMeshLODModel from the imported skeletal mesh as the new LOD in the selected mesh.
	if (SourceImportedResource->LODModels.Num() == 0)
	{
		return false;
	}

	// Names of root bones must match.
	// If the names of root bones don't match, the LOD Mesh does not share skeleton with base Mesh. 
	if (SourceSkeletalMesh->GetRefSkeleton().GetBoneName(0) != DestinationSkeletalMesh->GetRefSkeleton().GetBoneName(0))
	{
		UE_LOGF(LogLODUtilities, Error, "SkeletalMesh [%ls] FLODUtilities::SetCustomLOD: Root bone in LOD is '%ls' instead of '%ls'.\nImport failed.."
		, *DestinationSkeletalMesh->GetName()
		, *SourceSkeletalMesh->GetRefSkeleton().GetBoneName(0).ToString()
		, *DestinationSkeletalMesh->GetRefSkeleton().GetBoneName(0).ToString());
		return false;
	}
	// We do some checking here that for every bone in the mesh we just imported, it's in our base ref skeleton, and the parent is the same.
	for (int32 i = 0; i < SourceSkeletalMesh->GetRefSkeleton().GetRawBoneNum(); i++)
	{
		int32 LODBoneIndex = i;
		FName LODBoneName = SourceSkeletalMesh->GetRefSkeleton().GetBoneName(LODBoneIndex);
		int32 BaseBoneIndex = DestinationSkeletalMesh->GetRefSkeleton().FindBoneIndex(LODBoneName);
		if (BaseBoneIndex == INDEX_NONE)
		{
			UE_LOGF(LogLODUtilities, Error, "FLODUtilities::SetCustomLOD: Bone '%ls' not found in destination SkeletalMesh '%ls'.\nImport failed."
				, *LODBoneName.ToString()
				, *DestinationSkeletalMesh->GetName());
			return false;
		}

		if (i > 0)
		{
			int32 LODParentIndex = SourceSkeletalMesh->GetRefSkeleton().GetParentIndex(LODBoneIndex);
			FName LODParentName = SourceSkeletalMesh->GetRefSkeleton().GetBoneName(LODParentIndex);

			int32 BaseParentIndex = DestinationSkeletalMesh->GetRefSkeleton().GetParentIndex(BaseBoneIndex);
			FName BaseParentName = DestinationSkeletalMesh->GetRefSkeleton().GetBoneName(BaseParentIndex);

			if (LODParentName != BaseParentName)
			{
				UE_LOGF(LogLODUtilities, Error, "SkeletalMesh [%ls] FLODUtilities::SetCustomLOD: Bone '%ls' in LOD has parent '%ls' instead of '%ls'"
					, *DestinationSkeletalMesh->GetName()
					, *LODBoneName.ToString()
					, *LODParentName.ToString()
					, *BaseParentName.ToString());
				return false;
			}
		}
	}

	FSkeletalMeshLODModel& NewLODModel = SourceImportedResource->LODModels[0];

	// If this LOD is not the base LOD, we check all bones we have sockets on are present in it.
	if (LodIndex > 0)
	{
		const TArray<USkeletalMeshSocket*>& Sockets = DestinationSkeletalMesh->GetMeshOnlySocketList();

		for (int32 i = 0; i < Sockets.Num(); i++)
		{
			// Find bone index the socket is attached to.
			USkeletalMeshSocket* Socket = Sockets[i];
			int32 SocketBoneIndex = SourceSkeletalMesh->GetRefSkeleton().FindBoneIndex(Socket->BoneName);

			// If this LOD does not contain the socket bone, abort import.
			if (SocketBoneIndex == INDEX_NONE)
			{
				UE_LOGF(LogLODUtilities, Error, "FLODUtilities::SetCustomLOD: This LOD is missing bone '%ls' used by socket '%ls'.\nAborting import."
					, *Socket->BoneName.ToString()
					, *Socket->SocketName.ToString());
				return false;
			}
		}
	}

	{
		//The imported LOD is always in LOD 0 of the SourceSkeletalMesh
		const int32 SourceLODIndex = 0;
		if (SourceSkeletalMesh->HasMeshDescription(SourceLODIndex))
		{
			// Fix up the imported data bone indexes
			FSkeletalMeshImportData LODImportData;
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			SourceSkeletalMesh->LoadLODImportedData(SourceLODIndex, LODImportData);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			const int32 LODImportDataBoneNumber = LODImportData.RefBonesBinary.Num();
			//We want to create a remap array so we can fix all influence easily
			TArray<int32> ImportDataBoneRemap;
			ImportDataBoneRemap.AddZeroed(LODImportDataBoneNumber);
			//We generate a new RefBonesBinary array to replace the existing one
			TArray<SkeletalMeshImportData::FBone> RemapedRefBonesBinary;
			RemapedRefBonesBinary.AddZeroed(DestinationSkeletalMesh->GetRefSkeleton().GetNum());
			for (int32 ImportBoneIndex = 0; ImportBoneIndex < LODImportDataBoneNumber; ++ImportBoneIndex)
			{
				SkeletalMeshImportData::FBone& ImportedBone = LODImportData.RefBonesBinary[ImportBoneIndex];
				int32 LODBoneIndex = ImportBoneIndex;
				FName LODBoneName = FName(*FSkeletalMeshImportData::FixupBoneName(ImportedBone.Name));
				int32 BaseBoneIndex = DestinationSkeletalMesh->GetRefSkeleton().FindBoneIndex(LODBoneName);
				ImportDataBoneRemap[ImportBoneIndex] = BaseBoneIndex;
				if (BaseBoneIndex != INDEX_NONE)
				{
					RemapedRefBonesBinary[BaseBoneIndex] = ImportedBone;
					if (RemapedRefBonesBinary[BaseBoneIndex].ParentIndex != INDEX_NONE)
					{
						RemapedRefBonesBinary[BaseBoneIndex].ParentIndex = ImportDataBoneRemap[RemapedRefBonesBinary[BaseBoneIndex].ParentIndex];
					}
				}
			}
			//Copy the new RefBonesBinary over the existing one
			LODImportData.RefBonesBinary = RemapedRefBonesBinary;

			//Fix the influences
			bool bNeedShrinking = false;
			const int32 InfluenceNumber = LODImportData.Influences.Num();
			for (int32 InfluenceIndex = InfluenceNumber - 1; InfluenceIndex >= 0; --InfluenceIndex)
			{
				SkeletalMeshImportData::FRawBoneInfluence& Influence = LODImportData.Influences[InfluenceIndex];
				Influence.BoneIndex = ImportDataBoneRemap[Influence.BoneIndex];
				if (Influence.BoneIndex == INDEX_NONE)
				{
					const int32 DeleteCount = 1;
					LODImportData.Influences.RemoveAt(InfluenceIndex, DeleteCount, EAllowShrinking::No);
					bNeedShrinking = true;
				}
			}
			//Shrink the array if we have deleted at least one entry
			if (bNeedShrinking)
			{
				LODImportData.Influences.Shrink();
			}
			//Save the fix up remap bone index
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			SourceSkeletalMesh->SaveLODImportedData(SourceLODIndex, LODImportData);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		// Fix up the ActiveBoneIndices array.
		for (int32 ActiveIndex = 0; ActiveIndex < NewLODModel.ActiveBoneIndices.Num(); ActiveIndex++)
		{
			int32 LODBoneIndex = NewLODModel.ActiveBoneIndices[ActiveIndex];
			FName LODBoneName = SourceSkeletalMesh->GetRefSkeleton().GetBoneName(LODBoneIndex);
			int32 BaseBoneIndex = DestinationSkeletalMesh->GetRefSkeleton().FindBoneIndex(LODBoneName);
			checkSlow(BaseBoneIndex != INDEX_NONE);
			NewLODModel.ActiveBoneIndices[ActiveIndex] = static_cast<FBoneIndexType>(BaseBoneIndex);
		}

		// Fix up the chunk BoneMaps.
		for (int32 SectionIndex = 0; SectionIndex < NewLODModel.Sections.Num(); SectionIndex++)
		{
			FSkelMeshSection& Section = NewLODModel.Sections[SectionIndex];
			for (int32 BoneMapIndex = 0; BoneMapIndex < Section.BoneMap.Num(); BoneMapIndex++)
			{
				int32 LODBoneIndex = Section.BoneMap[BoneMapIndex];
				FName LODBoneName = SourceSkeletalMesh->GetRefSkeleton().GetBoneName(LODBoneIndex);
				int32 BaseBoneIndex = DestinationSkeletalMesh->GetRefSkeleton().FindBoneIndex(LODBoneName);
				checkSlow(BaseBoneIndex != INDEX_NONE);
				Section.BoneMap[BoneMapIndex] = static_cast<FBoneIndexType>(BaseBoneIndex);
			}
		}

		// Create the RequiredBones array in the LODModel from the ref skeleton.
		for (int32 RequiredBoneIndex = 0; RequiredBoneIndex < NewLODModel.RequiredBones.Num(); RequiredBoneIndex++)
		{
			FName LODBoneName = SourceSkeletalMesh->GetRefSkeleton().GetBoneName(NewLODModel.RequiredBones[RequiredBoneIndex]);
			int32 BaseBoneIndex = DestinationSkeletalMesh->GetRefSkeleton().FindBoneIndex(LODBoneName);
			if (BaseBoneIndex != INDEX_NONE)
			{
				NewLODModel.RequiredBones[RequiredBoneIndex] = static_cast<FBoneIndexType>(BaseBoneIndex);
			}
			else
			{
				NewLODModel.RequiredBones.RemoveAt(RequiredBoneIndex--);
			}
		}

		// Also sort the RequiredBones array to be strictly increasing.
		NewLODModel.RequiredBones.Sort();
		DestinationSkeletalMesh->GetRefSkeleton().EnsureParentsExistAndSort(NewLODModel.ActiveBoneIndices);
	}

	// To be extra-nice, we apply the difference between the root transform of the meshes to the verts.
	FMatrix44f LODToBaseTransform = FMatrix44f(SourceSkeletalMesh->GetRefPoseMatrix(0).InverseFast() * DestinationSkeletalMesh->GetRefPoseMatrix(0));

	for (int32 SectionIndex = 0; SectionIndex < NewLODModel.Sections.Num(); SectionIndex++)
	{
		FSkelMeshSection& Section = NewLODModel.Sections[SectionIndex];

		// Fix up soft verts.
		for (int32 i = 0; i < Section.SoftVertices.Num(); i++)
		{
			Section.SoftVertices[i].Position = LODToBaseTransform.TransformPosition(Section.SoftVertices[i].Position);
			Section.SoftVertices[i].TangentX = LODToBaseTransform.TransformVector(Section.SoftVertices[i].TangentX);
			Section.SoftVertices[i].TangentY = LODToBaseTransform.TransformVector(Section.SoftVertices[i].TangentY);
			Section.SoftVertices[i].TangentZ = LODToBaseTransform.TransformVector(Section.SoftVertices[i].TangentZ);
		}
	}

	TArray<FName> ExistingOriginalPerSectionMaterialImportName;
	bool bIsReImport = false;
	//Restore the LOD section data in case this LOD was reimport and some material match
	if (DestImportedResource->LODModels.IsValidIndex(LodIndex) && SourceImportedResource->LODModels.IsValidIndex(0))
	{
		if (DestinationSkeletalMesh->HasMeshDescription(LodIndex))
		{
			FSkeletalMeshImportData DestinationLODImportData;
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			DestinationSkeletalMesh->LoadLODImportedData(LodIndex, DestinationLODImportData);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			for (int32 SectionIndex = 0; SectionIndex < DestinationLODImportData.Materials.Num(); ++SectionIndex)
			{
				ExistingOriginalPerSectionMaterialImportName.Add(FName(*DestinationLODImportData.Materials[SectionIndex].MaterialImportName));
			}
		}
		bIsReImport = true;
	}

	// If we want to add this as a new LOD to this mesh - add to LODModels/LODInfo array.
	if (LodIndex == DestImportedResource->LODModels.Num())
	{
		DestImportedResource->LODModels.Add(new FSkeletalMeshLODModel());

		// Add element to LODInfo array.
		DestinationSkeletalMesh->AddLODInfo();
		check(DestinationSkeletalMesh->GetLODNum() == DestImportedResource->LODModels.Num());
	}

	const int32 SourceLODIndex = 0;
	if (SourceSkeletalMesh->HasMeshDescription(SourceLODIndex))
	{
		// Fix up the imported data bone indexes
		FSkeletalMeshImportData SourceLODImportData;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		SourceSkeletalMesh->LoadLODImportedData(SourceLODIndex, SourceLODImportData);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		FLODUtilities::FSkeletalMeshMatchImportedMaterialsParameters Parameters;
		Parameters.bIsReImport = bIsReImport;
		Parameters.LodIndex = LodIndex;
		Parameters.SkeletalMesh = DestinationSkeletalMesh;
		Parameters.ImportedMaterials = &SourceLODImportData.Materials;
		Parameters.ExistingOriginalPerSectionMaterialImportName = &ExistingOriginalPerSectionMaterialImportName;
		Parameters.CustomImportedLODModel = &NewLODModel;
		FLODUtilities::MatchImportedMaterials(Parameters);
	}

	// Set up LODMaterialMap to number of materials in new mesh.
	FSkeletalMeshLODInfo& LODInfo = *(DestinationSkeletalMesh->GetLODInfo(LodIndex));

	//Copy the build settings
	if (SourceSkeletalMesh->GetLODInfo(0))
	{
		const FSkeletalMeshLODInfo& ImportedLODInfo = *(SourceSkeletalMesh->GetLODInfo(0));
		LODInfo.BuildSettings = ImportedLODInfo.BuildSettings;
	}

	// Release all resources before replacing the model
	DestinationSkeletalMesh->PreEditChange(NULL);

	// Assign new FSkeletalMeshLODModel to desired slot in selected skeletal mesh.
	FSkeletalMeshLODModel::CopyStructure(&(DestImportedResource->LODModels[LodIndex]), &NewLODModel);
	
	//Copy the import data into the base skeletalmesh for the imported LOD
	FMeshDescription SourceMeshDescription;
	if (SourceSkeletalMesh->CloneMeshDescription(0, SourceMeshDescription))
	{
		DestinationSkeletalMesh->ModifyMeshDescription(LodIndex);
		DestinationSkeletalMesh->CreateMeshDescription(LodIndex, MoveTemp(SourceMeshDescription));
		DestinationSkeletalMesh->CommitMeshDescription(LodIndex);
	}


	// If this LOD had been generated previously by automatic mesh reduction, clear that flag.
	LODInfo.bHasBeenSimplified = false;
	if (DestinationSkeletalMesh->GetLODSettings() == nullptr || !DestinationSkeletalMesh->GetLODSettings()->HasValidSettings() || DestinationSkeletalMesh->GetLODSettings()->GetNumberOfSettings() <= LodIndex)
	{
		//Make sure any custom LOD have correct settings (no reduce)
		LODInfo.ReductionSettings.NumOfTrianglesPercentage = 1.0f;
		LODInfo.ReductionSettings.MaxNumOfTriangles = MAX_uint32;
		LODInfo.ReductionSettings.MaxNumOfTrianglesPercentage = MAX_uint32;
		LODInfo.ReductionSettings.NumOfVertPercentage = 1.0f;
		LODInfo.ReductionSettings.MaxNumOfVerts = MAX_uint32;
		LODInfo.ReductionSettings.MaxNumOfVertsPercentage = MAX_uint32;
		LODInfo.ReductionSettings.MaxDeviationPercentage = 0.0f;
	}

	// Set LOD source filename
	DestinationSkeletalMesh->GetLODInfo(LodIndex)->SourceImportFilename = UAssetImportData::SanitizeImportFilename(SourceDataFilename, nullptr);
	DestinationSkeletalMesh->GetLODInfo(LodIndex)->bImportWithBaseMesh = false;

	ReapplyClothing();

	ReapplyCustomImportedMorphTarget();

	ReapplyAlternateSkinning();
	
	// Notification of success
	FNotificationInfo NotificationInfo(FText::GetEmpty());
	NotificationInfo.Text = FText::Format(NSLOCTEXT("UnrealEd", "LODImportSuccessful", "Mesh for LOD {0} imported successfully!"), FText::AsNumber(LodIndex));
	NotificationInfo.ExpireDuration = 5.0f;
	FSlateNotificationManager::Get().AddNotification(NotificationInfo);
	
	return true;
}

/** Given three direction vectors, indicates if A and B are on the same 'side' of Vec. */
bool VectorsOnSameSide(const FVector2D& Vec, const FVector2D& A, const FVector2D& B)
{
	return !(((B.Y - A.Y)*(Vec.X - A.X)) + ((A.X - B.X)*(Vec.Y - A.Y)) < 0.0f);
}

double PointToSegmentDistanceSquare(const FVector2D& A, const FVector2D& B, const FVector2D& P)
{
	return FVector2D::DistSquared(P, FMath::ClosestPointOnSegment2D(P, A, B));
}

/** Return true if P is within triangle created by A, B and C. */
bool PointInTriangle(const FVector2D& A, const FVector2D& B, const FVector2D& C, const FVector2D& P)
{
	//If the point is on a triangle point we consider the point inside the triangle
	
	if (P.Equals(A) || P.Equals(B) || P.Equals(C))
	{
		return true;
	}
	// If its on the same side as the remaining vert for all edges, then its inside.	
	if (VectorsOnSameSide(A, B, P) &&
		VectorsOnSameSide(B, C, P) &&
		VectorsOnSameSide(C, A, P))
	{
		return true;
	}

	//Make sure point on the edge are count inside the triangle
	if (PointToSegmentDistanceSquare(A, B, P) <= KINDA_SMALL_NUMBER)
	{
		return true;
	}
	if (PointToSegmentDistanceSquare(B, C, P) <= KINDA_SMALL_NUMBER)
	{
		return true;
	}
	if (PointToSegmentDistanceSquare(C, A, P) <= KINDA_SMALL_NUMBER)
	{
		return true;
	}
	return false;
}

/** Given three direction vectors, indicates if A and B are on the same 'side' of Vec. */
bool VectorsOnSameSide(const FVector3f& Vec, const FVector3f& A, const FVector3f& B, const float SameSideDotProductEpsilon)
{
	const FVector CrossA = FVector(Vec ^ A);
	const FVector CrossB = FVector(Vec ^ B);
	double DotWithEpsilon = SameSideDotProductEpsilon + (CrossA | CrossB);
	return !(DotWithEpsilon < 0.0f);
}

/** Util to see if P lies within triangle created by A, B and C. */
bool PointInTriangle(const FVector3f& A, const FVector3f& B, const FVector3f& C, const FVector3f& P)
{
	// Cross product indicates which 'side' of the vector the point is on
	// If its on the same side as the remaining vert for all edges, then its inside.	
	if (VectorsOnSameSide(B - A, P - A, C - A, KINDA_SMALL_NUMBER) &&
		VectorsOnSameSide(C - B, P - B, A - B, KINDA_SMALL_NUMBER) &&
		VectorsOnSameSide(A - C, P - C, B - C, KINDA_SMALL_NUMBER))
	{
		return true;
	}
	return false;
}

FVector3f GetBaryCentric(const FVector3f& Point, const FVector3f& A, const FVector3f& B, const FVector3f& C, const TCHAR* DebugContext)
{
	// Compute the normal of the triangle
	const FVector3f TriNorm = (B - A) ^ (C - A);

	//check collinearity of A,B,C
	if (TriNorm.SizeSquared() <= SMALL_NUMBER)
	{
		float DistA = FVector3f::DistSquared(Point, A);
		float DistB = FVector3f::DistSquared(Point, B);
		float DistC = FVector3f::DistSquared(Point, C);
		if(DistA <= DistB && DistA <= DistC)
		{
			return FVector3f(1.0f, 0.0f, 0.0f);
		}
		if (DistB <= DistC)
		{
			return FVector3f(0.0f, 1.0f, 0.0f);
		}
		return FVector3f(0.0f, 0.0f, 1.0f);
	}
	FVector ToRet(0, 0, 0);
	if (!FMath::ComputeBarycentricTri((FVector)Point, (FVector)A, (FVector)B, (FVector)C, ToRet, UE_DOUBLE_SMALL_NUMBER))
	{
		UE_LOGF(LogUnrealMath, Warning, "SkeletalMesh [%ls]: Small triangle detected in FMath::ComputeBaryCentric2D(); can't compute valid barycentric coordinate.", DebugContext ? DebugContext : TEXT("Unknown Source"));
	}
	return (FVector3f)ToRet;
}

struct FTriangleElement
{
	FBox2D UVsBound;
	FBoxCenterAndExtent PositionBound;
	TArray<FSoftSkinVertex> Vertices;
	TArray<uint32> Indexes;
	uint32 TriangleIndex;
};

bool FindTriangleUVMatch(const FVector2D& TargetUV, const TArray<FTriangleElement>& Triangles, const TArray<uint32>& QuadTreeTriangleResults, TArray<uint32>& MatchTriangleIndexes)
{
	for (uint32 TriangleIndex : QuadTreeTriangleResults)
	{
		const FTriangleElement& TriangleElement = Triangles[TriangleIndex];
		if (PointInTriangle(FVector2D(TriangleElement.Vertices[0].UVs[0]), FVector2D(TriangleElement.Vertices[1].UVs[0]), FVector2D(TriangleElement.Vertices[2].UVs[0]), TargetUV))
		{
			MatchTriangleIndexes.Add(TriangleIndex);
		}
		TriangleIndex++;
	}
	return MatchTriangleIndexes.Num() == 0 ? false : true;
}

bool FindTrianglePositionMatch(const FVector& Position, const TArray<FTriangleElement>& Triangles, const TArray<FTriangleElement>& OcTreeTriangleResults, TArray<uint32>& MatchTriangleIndexes)
{
	for (const FTriangleElement& Triangle : OcTreeTriangleResults)
	{
		uint32 TriangleIndex = Triangle.TriangleIndex;
		const FTriangleElement& TriangleElement = Triangles[TriangleIndex];
		if (PointInTriangle((FVector3f)TriangleElement.Vertices[0].Position, (FVector3f)TriangleElement.Vertices[1].Position, (FVector3f)TriangleElement.Vertices[2].Position, (FVector3f)Position))
		{
			MatchTriangleIndexes.Add(TriangleIndex);
		}
		TriangleIndex++;
	}
	return MatchTriangleIndexes.Num() == 0 ? false : true;
}

struct FTargetMatch
{
	float BarycentricWeight[3]; //The weight we use to interpolate the TARGET data
	uint32 Indices[3]; //BASE Index of the triangle vertice
	
	//Default constructor
	FTargetMatch()
	{
		BarycentricWeight[0] = BarycentricWeight[1] = BarycentricWeight[2] = 0.0f;
		Indices[0] = Indices[1] = Indices[2] = INDEX_NONE;
	}
};

void ProjectTargetOnBase(const TArray<FSoftSkinVertex>& BaseVertices, const TArray<TArray<uint32>>& PerSectionBaseTriangleIndices,
						 TArray<FTargetMatch>& TargetMatchData, const TArray<FSkelMeshSection>& TargetSections, const TArray<int32>& TargetSectionMatchBaseIndex, const TCHAR* DebugContext)
{
	bool bNoMatchMsgDone = false;
	bool bNoUVsMsgDisplayed = false;
	TArray<FTriangleElement> Triangles;
	//Project section target vertices on match base section using the UVs coordinates
	for (int32 SectionIndex = 0; SectionIndex < TargetSections.Num(); ++SectionIndex)
	{
		//Use the remap base index in case some sections disappear during the reduce phase
		int32 BaseSectionIndex = TargetSectionMatchBaseIndex[SectionIndex];
		if (BaseSectionIndex == INDEX_NONE || !PerSectionBaseTriangleIndices.IsValidIndex(BaseSectionIndex) || PerSectionBaseTriangleIndices[BaseSectionIndex].Num() < 1)
		{
			continue;
		}
		//Target vertices for the Section
		const TArray<FSoftSkinVertex>& TargetVertices = TargetSections[SectionIndex].SoftVertices;
		//Base Triangle indices for the matched base section
		const TArray<uint32>& BaseTriangleIndices = PerSectionBaseTriangleIndices[BaseSectionIndex];
		FBox2D BaseMeshUVBound(EForceInit::ForceInit);
		FBox BaseMeshPositionBound(EForceInit::ForceInit);
		//Fill the triangle element to speed up the triangle research
		Triangles.Reset(BaseTriangleIndices.Num() / 3);
		for (uint32 TriangleIndex = 0; TriangleIndex < (uint32)BaseTriangleIndices.Num(); TriangleIndex += 3)
		{
			FTriangleElement TriangleElement;
			TriangleElement.UVsBound.Init();
			for (int32 Corner = 0; Corner < 3; ++Corner)
			{
				uint32 CornerIndice = BaseTriangleIndices[TriangleIndex + Corner];
				check(BaseVertices.IsValidIndex(CornerIndice));
				const FSoftSkinVertex& BaseVertex = BaseVertices[CornerIndice];
				TriangleElement.Indexes.Add(CornerIndice);
				TriangleElement.Vertices.Add(BaseVertex);
				TriangleElement.UVsBound += FVector2D(BaseVertex.UVs[0]);
				BaseMeshPositionBound += (FVector)BaseVertex.Position;
			}
			BaseMeshUVBound += TriangleElement.UVsBound;
			TriangleElement.TriangleIndex = Triangles.Num();
			Triangles.Add(TriangleElement);
		}
		if (BaseMeshUVBound.GetExtent().IsNearlyZero())
		{
			if(!bNoUVsMsgDisplayed)
			{
				UE_LOGF(LogLODUtilities, Display, "SkeletalMesh [%ls] Remap morph target: Cannot remap morph target because source UVs are missings.", DebugContext ? DebugContext : TEXT("Unknown Source"));
				bNoUVsMsgDisplayed = true;
			}
			continue;
		}
		//Setup the Quad tree
		float UVsQuadTreeMinSize = 0.001f;
		TQuadTree<uint32, 100> QuadTree(BaseMeshUVBound, UVsQuadTreeMinSize);
		for (FTriangleElement& TriangleElement : Triangles)
		{
			QuadTree.Insert(TriangleElement.TriangleIndex, TriangleElement.UVsBound, DebugContext);
		}
		//Retrieve all triangle that are close to our point, let get 5% of UV extend
		double DistanceThreshold = BaseMeshUVBound.GetExtent().Size()*0.05;
		//Find a match triangle for every target vertices
		TArray<uint32> QuadTreeTriangleResults;
		QuadTreeTriangleResults.Reserve(Triangles.Num() / 10); //Reserve 10% to speed up the query
		for (uint32 TargetVertexIndex = 0; TargetVertexIndex < (uint32)TargetVertices.Num(); ++TargetVertexIndex)
		{
			FVector2D TargetUV = FVector2D(TargetVertices[TargetVertexIndex].UVs[0]);
			//Reset the last data without flushing the memmery allocation
			QuadTreeTriangleResults.Reset();
			const uint32 FullTargetIndex = TargetSections[SectionIndex].BaseVertexIndex + TargetVertexIndex;
			//Make sure the array is allocate properly
			if (!TargetMatchData.IsValidIndex(FullTargetIndex))
			{
				continue;
			}
			//Set default data for the target match, in case we cannot found a match
			FTargetMatch& TargetMatch = TargetMatchData[FullTargetIndex];
			for (int32 Corner = 0; Corner < 3; ++Corner)
			{
				TargetMatch.Indices[Corner] = INDEX_NONE;
				TargetMatch.BarycentricWeight[Corner] = 0.3333f; //The weight will be use to found the proper delta
			}

			FVector2D Extent(DistanceThreshold, DistanceThreshold);
			FBox2D CurBox(TargetUV - Extent, TargetUV + Extent);
			while (QuadTreeTriangleResults.Num() <= 0)
			{
				QuadTree.GetElements(CurBox, QuadTreeTriangleResults);
				Extent *= 2;
				CurBox = FBox2D(TargetUV - Extent, TargetUV + Extent);
			}

			auto GetDistancePointToBaseTriangle = [&Triangles, &TargetVertices, &TargetVertexIndex](const uint32 BaseTriangleIndex)->double
			{
				FTriangleElement& CandidateTriangle = Triangles[BaseTriangleIndex];
				return FVector::DistSquared(FMath::ClosestPointOnTriangleToPoint((FVector)TargetVertices[TargetVertexIndex].Position, (FVector)CandidateTriangle.Vertices[0].Position, (FVector)CandidateTriangle.Vertices[1].Position, (FVector)CandidateTriangle.Vertices[2].Position), (FVector)TargetVertices[TargetVertexIndex].Position);
			};

			auto FailSafeUnmatchVertex = [&GetDistancePointToBaseTriangle, &QuadTreeTriangleResults](uint32 &OutIndexMatch)->bool
			{
				bool bFoundMatch = false;
				double ClosestTriangleDistSquared = std::numeric_limits<double>::max();
				for (uint32 MatchTriangleIndex : QuadTreeTriangleResults)
				{
					double TriangleDistSquared = GetDistancePointToBaseTriangle(MatchTriangleIndex);
					if (TriangleDistSquared < ClosestTriangleDistSquared)
					{
						ClosestTriangleDistSquared = TriangleDistSquared;
						OutIndexMatch = MatchTriangleIndex;
						bFoundMatch = true;
					}
				}
				return bFoundMatch;
			};

			//Find all Triangles that contain the Target UV
			if (QuadTreeTriangleResults.Num() > 0)
			{
				TArray<uint32> MatchTriangleIndexes;
				uint32 FoundIndexMatch = INDEX_NONE;
				if(!FindTriangleUVMatch(TargetUV, Triangles, QuadTreeTriangleResults, MatchTriangleIndexes))
				{
					if (!FailSafeUnmatchVertex(FoundIndexMatch))
					{
						//We should always have a match
						if (!bNoMatchMsgDone)
						{
							UE_LOGF(LogLODUtilities, Warning, "Reduce LOD, remap morph target: Cannot find a triangle from the base LOD that contain a vertex UV in the target LOD. Remap morph target quality will be lower.");
							bNoMatchMsgDone = true;
						}
						continue;
					}
				}
				double ClosestTriangleDistSquared = std::numeric_limits<double>::max();
				if (MatchTriangleIndexes.Num() == 1)
				{
					//One match, this mean no mirror UVs simply take the single match
					FoundIndexMatch = MatchTriangleIndexes[0];
					ClosestTriangleDistSquared = GetDistancePointToBaseTriangle(FoundIndexMatch);
				}
				else
				{
					//Geometry can use mirror so the UVs are not unique. Use the closest match triangle to the point to find the best match
					for (uint32 MatchTriangleIndex : MatchTriangleIndexes)
					{
						double TriangleDistSquared = GetDistancePointToBaseTriangle(MatchTriangleIndex);
						if (TriangleDistSquared < ClosestTriangleDistSquared)
						{
							ClosestTriangleDistSquared = TriangleDistSquared;
							FoundIndexMatch = MatchTriangleIndex;
						}
					}
				}

				//FAIL SAFE, make sure we have a match that make sense
				//Use the mesh section geometry bound extent (10% of it) to validate we are close enough.
				if (ClosestTriangleDistSquared > BaseMeshPositionBound.GetExtent().SizeSquared()*0.1f)
				{
					//Executing fail safe, if the UVs are too much off because of the reduction, use the closest distance to polygons to find the match
					//This path is not optimize and should not happen often.
					FailSafeUnmatchVertex(FoundIndexMatch);
				}

				//We should always have a valid match at this point
				check(FoundIndexMatch != INDEX_NONE);
				FTriangleElement& BestTriangle = Triangles[FoundIndexMatch];
				//Found the surface area of the 3 barycentric triangles from the UVs
				FVector3f BarycentricWeight;
				BarycentricWeight = GetBaryCentric(FVector3f(FVector2f(TargetUV), 0.0f), FVector3f(BestTriangle.Vertices[0].UVs[0], 0.0f), FVector3f(BestTriangle.Vertices[1].UVs[0], 0.0f), FVector3f(BestTriangle.Vertices[2].UVs[0], 0.0f), DebugContext);	// LWC_TODO: Precision loss
				//Fill the target match
				for (int32 Corner = 0; Corner < 3; ++Corner)
				{
					TargetMatch.Indices[Corner] = BestTriangle.Indexes[Corner];
					TargetMatch.BarycentricWeight[Corner] = BarycentricWeight[Corner]; //The weight will be use to found the proper delta
				}
			}
			else
			{
				if (!bNoMatchMsgDone)
				{
					UE_LOGF(LogLODUtilities, Warning, "Reduce LOD, remap morph target: Cannot find a triangle from the base LOD that contain a vertex UV in the target LOD. Remap morph target quality will be lower.");
					bNoMatchMsgDone = true;
				}
				continue;
			}
		}
	}
}

void CreateLODMorphTarget(USkeletalMesh* SkeletalMesh, const FInlineReductionDataParameter& InlineReductionDataParameter, int32 SourceLOD, int32 DestinationLOD, const TMap<UMorphTarget *, TMap<uint32, uint32>>& PerMorphTargetBaseIndexToMorphTargetDelta, const TMap<uint32, TArray<uint32>>& BaseMorphIndexToTargetIndexList, const TArray<FSoftSkinVertex>& TargetVertices, const TArray<FTargetMatch>& TargetMatchData)
{
	FSkeletalMeshModel* SkeletalMeshModel = SkeletalMesh->GetImportedModel();
	const FSkeletalMeshLODModel& TargetLODModel = SkeletalMeshModel->LODModels[DestinationLOD];

	bool bInitializeMorphData = false;

	for (UMorphTarget *MorphTarget : SkeletalMesh->GetMorphTargets())
	{
		if (!MorphTarget->HasDataForLOD(SourceLOD))
		{
			continue;
		}
		bool bUseBaseMorphDelta = SourceLOD == DestinationLOD && InlineReductionDataParameter.bIsDataValid && InlineReductionDataParameter.InlineOriginalSrcMorphTargetData.Contains(MorphTarget->GetFullName());

		const TArray<FMorphTargetDelta> *BaseMorphDeltas = bUseBaseMorphDelta ? InlineReductionDataParameter.InlineOriginalSrcMorphTargetData.Find(MorphTarget->GetFullName()) : nullptr;
		if (BaseMorphDeltas == nullptr || BaseMorphDeltas->Num() <= 0)
		{
			bUseBaseMorphDelta = false;
		}

		const TMap<uint32, uint32>& BaseIndexToMorphTargetDelta = PerMorphTargetBaseIndexToMorphTargetDelta[MorphTarget];
		TArray<FMorphTargetDelta> NewMorphTargetDeltas;
		TSet<uint32> CreatedTargetIndex;
		TMap<FVector3f, TArray<uint32>> MorphTargetPerPosition;
		const FMorphTargetLODModel& BaseMorphModel = MorphTarget->GetMorphLODModels()[SourceLOD];
		//Iterate each original morph target source index to fill the NewMorphTargetDeltas array with the TargetMatchData.
		const TArray<FMorphTargetDelta>& Vertices = bUseBaseMorphDelta ? *BaseMorphDeltas : BaseMorphModel.Vertices;
		for (uint32 MorphDeltaIndex = 0; MorphDeltaIndex < (uint32)(Vertices.Num()); ++MorphDeltaIndex)
		{
			const FMorphTargetDelta& MorphDelta = Vertices[MorphDeltaIndex];
			const TArray<uint32>* TargetIndexesPtr = BaseMorphIndexToTargetIndexList.Find(MorphDelta.SourceIdx);
			if (TargetIndexesPtr == nullptr)
			{
				continue;
			}
			const TArray<uint32>& TargetIndexes = *TargetIndexesPtr;
			for (int32 MorphTargetIndex = 0; MorphTargetIndex < TargetIndexes.Num(); ++MorphTargetIndex)
			{
				uint32 TargetIndex = TargetIndexes[MorphTargetIndex];
				if (CreatedTargetIndex.Contains(TargetIndex))
				{
					continue;
				}
				CreatedTargetIndex.Add(TargetIndex);
				const FVector3f& SearchPosition = TargetVertices[TargetIndex].Position;
				FMorphTargetDelta MatchMorphDelta;
				MatchMorphDelta.SourceIdx = TargetIndex;

				const FTargetMatch& TargetMatch = TargetMatchData[TargetIndex];

				//Find the Position/tangent delta for the MatchMorphDelta using the barycentric weight
				MatchMorphDelta.PositionDelta = FVector3f::ZeroVector;
				MatchMorphDelta.TangentZDelta = FVector3f::ZeroVector;
				for (int32 Corner = 0; Corner < 3; ++Corner)
				{
					const uint32* BaseMorphTargetIndexPtr = BaseIndexToMorphTargetDelta.Find(TargetMatch.Indices[Corner]);
					if (BaseMorphTargetIndexPtr != nullptr && Vertices.IsValidIndex(*BaseMorphTargetIndexPtr))
					{
						const FMorphTargetDelta& BaseMorphTargetDelta = Vertices[*BaseMorphTargetIndexPtr];
						FVector3f BasePositionDelta = !BaseMorphTargetDelta.PositionDelta.ContainsNaN() ? BaseMorphTargetDelta.PositionDelta : FVector3f(0.0f);
						FVector3f BaseTangentZDelta = !BaseMorphTargetDelta.TangentZDelta.ContainsNaN() ? BaseMorphTargetDelta.TangentZDelta : FVector3f(0.0f);
						MatchMorphDelta.PositionDelta += BasePositionDelta * TargetMatch.BarycentricWeight[Corner];
						MatchMorphDelta.TangentZDelta += BaseTangentZDelta * TargetMatch.BarycentricWeight[Corner];
					}
					ensure(!MatchMorphDelta.PositionDelta.ContainsNaN());
					ensure(!MatchMorphDelta.TangentZDelta.ContainsNaN());
				}

				//Make sure all morph delta that are at the same position use the same delta to avoid hole in the geometry
				TArray<uint32> *MorphTargetsIndexUsingPosition = nullptr;
				MorphTargetsIndexUsingPosition = MorphTargetPerPosition.Find(SearchPosition);
				if (MorphTargetsIndexUsingPosition != nullptr)
				{
					//Get the maximum position/tangent delta for the existing matched morph delta
					FVector3f PositionDelta = MatchMorphDelta.PositionDelta;
					FVector3f TangentZDelta = MatchMorphDelta.TangentZDelta;
					for (uint32 ExistingMorphTargetIndex : *MorphTargetsIndexUsingPosition)
					{
						const FMorphTargetDelta& ExistingMorphDelta = NewMorphTargetDeltas[ExistingMorphTargetIndex];
						PositionDelta = PositionDelta.SizeSquared() > ExistingMorphDelta.PositionDelta.SizeSquared() ? PositionDelta : ExistingMorphDelta.PositionDelta;
						TangentZDelta = TangentZDelta.SizeSquared() > ExistingMorphDelta.TangentZDelta.SizeSquared() ? TangentZDelta : ExistingMorphDelta.TangentZDelta;
					}
					//Update all MorphTarget that share the same position.
					for (uint32 ExistingMorphTargetIndex : *MorphTargetsIndexUsingPosition)
					{
						FMorphTargetDelta& ExistingMorphDelta = NewMorphTargetDeltas[ExistingMorphTargetIndex];
						ExistingMorphDelta.PositionDelta = PositionDelta;
						ExistingMorphDelta.TangentZDelta = TangentZDelta;
					}
					MatchMorphDelta.PositionDelta = PositionDelta;
					MatchMorphDelta.TangentZDelta = TangentZDelta;
					MorphTargetsIndexUsingPosition->Add(NewMorphTargetDeltas.Num());
				}
				else
				{
					MorphTargetPerPosition.Add(TargetVertices[TargetIndex].Position).Add(NewMorphTargetDeltas.Num());
				}
				NewMorphTargetDeltas.Add(MatchMorphDelta);
			}
		}
		
		//Register the new morph target on the target LOD
		MorphTarget->PopulateDeltas(NewMorphTargetDeltas, DestinationLOD, TargetLODModel.Sections, false, true);
		if (MorphTarget->HasValidData())
		{
			bInitializeMorphData |= SkeletalMesh->RegisterMorphTarget(MorphTarget, false);
		}
	}

	if (bInitializeMorphData)
	{
		SkeletalMesh->InitMorphTargetsAndRebuildRenderData();
	}
}

void FLODUtilities::ClearGeneratedMorphTarget(USkeletalMesh* SkeletalMesh, int32 TargetLOD)
{
	check(SkeletalMesh);
	FSkeletalMeshModel* SkeletalMeshResource = SkeletalMesh->GetImportedModel();
	if (!SkeletalMeshResource ||
		!SkeletalMeshResource->LODModels.IsValidIndex(TargetLOD))
	{
		//Abort clearing 
		return;
	}

	const FSkeletalMeshLODModel& TargetLODModel = SkeletalMeshResource->LODModels[TargetLOD];
	//Make sure we have some morph for this LOD
	for (UMorphTarget *MorphTarget : SkeletalMesh->GetMorphTargets())
	{
		if (!MorphTarget->HasDataForLOD(TargetLOD))
		{
			continue;
		}

		//if (MorphTarget->MorphLODModels[TargetLOD].bGeneratedByEngine)
		{
			MorphTarget->GetMorphLODModels()[TargetLOD].Reset();

			// if this is the last one, we can remove empty ones
			if (TargetLOD == MorphTarget->GetMorphLODModels().Num() - 1)
			{
				MorphTarget->RemoveEmptyMorphTargets();
			}
		}
	}
}

void FLODUtilities::ApplyMorphTargetsToLOD(USkeletalMesh* SkeletalMesh, int32 SourceLOD, int32 DestinationLOD, const FInlineReductionDataParameter& InlineReductionDataParameter)
{
	check(SkeletalMesh);
	FSkeletalMeshModel* SkeletalMeshResource = SkeletalMesh->GetImportedModel();
	if (!SkeletalMeshResource ||
		!SkeletalMeshResource->LODModels.IsValidIndex(SourceLOD) ||
		!SkeletalMeshResource->LODModels.IsValidIndex(DestinationLOD) ||
		SourceLOD > DestinationLOD)
	{
		//Cannot reduce if the source model is missing or we reduce from a higher index LOD
		return;
	}

	FSkeletalMeshLODModel& SourceLODModel = SkeletalMeshResource->LODModels[SourceLOD];
	bool bReduceBaseLOD = DestinationLOD == SourceLOD && InlineReductionDataParameter.bIsDataValid;
	if (!bReduceBaseLOD && SourceLOD == DestinationLOD)
	{
		//Abort remapping of morph target since the data is missing
		return;
	}

	//Make sure we have some morph for this LOD
	bool bContainsMorphTargets = false;
	for (UMorphTarget* MorphTarget : SkeletalMesh->GetMorphTargets())
	{
		if (MorphTarget->HasDataForLOD(SourceLOD))
		{
			bContainsMorphTargets = true;
		}
	}
	if (!bContainsMorphTargets)
	{
		//No morph target to remap
		return;
	}

	const FSkeletalMeshLODModel& BaseLODModel = bReduceBaseLOD ? InlineReductionDataParameter.InlineOriginalSrcModel : SkeletalMeshResource->LODModels[SourceLOD];
	const FSkeletalMeshLODInfo* BaseLODInfo = SkeletalMesh->GetLODInfo(SourceLOD);
	const FSkeletalMeshLODModel& TargetLODModel = SkeletalMeshResource->LODModels[DestinationLOD];
	const FSkeletalMeshLODInfo* TargetLODInfo = SkeletalMesh->GetLODInfo(DestinationLOD);

	TArray<int32> BaseLODMaterialMap = BaseLODInfo ? BaseLODInfo->LODMaterialMap : TArray<int32>();
	TArray<int32> TargetLODMaterialMap = TargetLODInfo ? TargetLODInfo->LODMaterialMap : TArray<int32>();

	auto InternalGetSectionMaterialIndex = [](const FSkeletalMeshLODModel& LODModel, int32 SectionIndex)->int32
	{
		if (!LODModel.Sections.IsValidIndex(SectionIndex))
		{
			return 0;
		}
		return LODModel.Sections[SectionIndex].MaterialIndex;
	};

	auto GetBaseSectionMaterialIndex = [&BaseLODModel, &InternalGetSectionMaterialIndex](int32 SectionIndex)->int32
	{
		return InternalGetSectionMaterialIndex(BaseLODModel, SectionIndex);
	};

	auto GetTargetSectionMaterialIndex = [&TargetLODModel, &InternalGetSectionMaterialIndex](int32 SectionIndex)->int32
	{
		return InternalGetSectionMaterialIndex(TargetLODModel, SectionIndex);
	};

	//We have to match target sections index with the correct base section index. Reduced LODs can contain a different number of sections than the base LOD
	TArray<int32> TargetSectionMatchBaseIndex;
	//Initialize the array to INDEX_NONE
	TargetSectionMatchBaseIndex.AddUninitialized(TargetLODModel.Sections.Num());
	for (int32 TargetSectionIndex = 0; TargetSectionIndex < TargetLODModel.Sections.Num(); ++TargetSectionIndex)
	{
		TargetSectionMatchBaseIndex[TargetSectionIndex] = INDEX_NONE;
	}
	TBitArray<> BaseSectionMatch;
	BaseSectionMatch.Init(false, BaseLODModel.Sections.Num());
	//Find corresponding section indices from Source LOD for Target LOD
	for (int32 TargetSectionIndex = 0; TargetSectionIndex < TargetLODModel.Sections.Num(); ++TargetSectionIndex)
	{
		int32 TargetSectionMaterialIndex = GetTargetSectionMaterialIndex(TargetSectionIndex);
		for (int32 BaseSectionIndex = 0; BaseSectionIndex < BaseLODModel.Sections.Num(); ++BaseSectionIndex)
		{
			if (BaseSectionMatch[BaseSectionIndex])
			{
				continue;
			}
			int32 BaseSectionMaterialIndex = GetBaseSectionMaterialIndex(BaseSectionIndex);
			if (TargetSectionMaterialIndex == BaseSectionMaterialIndex)
			{
				TargetSectionMatchBaseIndex[TargetSectionIndex] = BaseSectionIndex;
				BaseSectionMatch[BaseSectionIndex] = true;
				break;
			}
		}
	}
	//We should have match all the target sections
	if (TargetSectionMatchBaseIndex.Contains(INDEX_NONE))
	{
		//This case is not fatal but need attention.
		//Because of the chunking its possible a generated LOD end up with more sections.
		UE_ASSET_LOG(LogLODUtilities, Display, SkeletalMesh, TEXT("FLODUtilities::ApplyMorphTargetsToLOD: The target contain more section then the source. Extra sections will not be affected by morph targets remap"));
	}
	TArray<FSoftSkinVertex> BaseVertices;
	TArray<FSoftSkinVertex> TargetVertices;
	BaseLODModel.GetVertices(BaseVertices);
	TargetLODModel.GetVertices(TargetVertices);
	//Create the base triangle indices per section
	TArray<TArray<uint32>> BaseTriangleIndices;
	int32 SectionCount = BaseLODModel.Sections.Num();
	BaseTriangleIndices.AddDefaulted(SectionCount);
	for (int32 SectionIndex = 0; SectionIndex < SectionCount; ++SectionIndex)
	{
		const FSkelMeshSection& Section = BaseLODModel.Sections[SectionIndex];
		uint32 TriangleCount = Section.NumTriangles;
		for (uint32 TriangleIndex = 0; TriangleIndex < TriangleCount; ++TriangleIndex)
		{
			for (uint32 PointIndex = 0; PointIndex < 3; PointIndex++)
			{
				uint32 IndexBufferValue = BaseLODModel.IndexBuffer[Section.BaseIndex + ((TriangleIndex * 3) + PointIndex)];
				BaseTriangleIndices[SectionIndex].Add(IndexBufferValue);
			}
		}
	}
	//Every target vertices match a Base LOD triangle, we also want the barycentric weight of the triangle match. All this done using the UVs
	TArray<FTargetMatch> TargetMatchData;
	TargetMatchData.AddDefaulted(TargetVertices.Num());
	//Match all target vertices to a Base triangle Using UVs.
	ProjectTargetOnBase(BaseVertices, BaseTriangleIndices, TargetMatchData, TargetLODModel.Sections, TargetSectionMatchBaseIndex, *SkeletalMesh->GetName());
	//Helper to retrieve the FMorphTargetDelta from the BaseIndex
	TMap<UMorphTarget *, TMap<uint32, uint32>> PerMorphTargetBaseIndexToMorphTargetDelta;
	//Create a map from BaseIndex to a list of match target index for all base morph target point
	TMap<uint32, TArray<uint32>> BaseMorphIndexToTargetIndexList;
	for (UMorphTarget *MorphTarget : SkeletalMesh->GetMorphTargets())
	{
		if (!MorphTarget->HasDataForLOD(SourceLOD))
		{
			continue;
		}

		bool bUseTempMorphDelta = SourceLOD == DestinationLOD && bReduceBaseLOD && InlineReductionDataParameter.InlineOriginalSrcMorphTargetData.Contains(MorphTarget->GetFullName());
		const TArray<FMorphTargetDelta> *TempMorphDeltas = bUseTempMorphDelta ? InlineReductionDataParameter.InlineOriginalSrcMorphTargetData.Find(MorphTarget->GetFullName()) : nullptr;
		if (TempMorphDeltas == nullptr || TempMorphDeltas->Num() <= 0)
		{
			bUseTempMorphDelta = false;
		}

		TMap<uint32, uint32>& BaseIndexToMorphTargetDelta = PerMorphTargetBaseIndexToMorphTargetDelta.FindOrAdd(MorphTarget);
		const FMorphTargetLODModel& BaseMorphModel = MorphTarget->GetMorphLODModels()[SourceLOD];
		const TArray<FMorphTargetDelta>& Vertices = bUseTempMorphDelta ? *TempMorphDeltas : BaseMorphModel.Vertices;
		for (uint32 MorphDeltaIndex = 0; MorphDeltaIndex < (uint32)(Vertices.Num()); ++MorphDeltaIndex)
		{
			const FMorphTargetDelta& MorphDelta = Vertices[MorphDeltaIndex];
			BaseIndexToMorphTargetDelta.Add(MorphDelta.SourceIdx, MorphDeltaIndex);
			//Iterate the targetmatch data so we can store which target indexes is impacted by this morph delta.
			for (int32 TargetIndex = 0; TargetIndex < TargetMatchData.Num(); ++TargetIndex)
			{
				const FTargetMatch& TargetMatch = TargetMatchData[TargetIndex];
				if (TargetMatch.Indices[0] == INDEX_NONE)
				{
					//In case this vertex did not found a triangle match
					continue;
				}
				if (TargetMatch.Indices[0] == MorphDelta.SourceIdx || TargetMatch.Indices[1] == MorphDelta.SourceIdx || TargetMatch.Indices[2] == MorphDelta.SourceIdx)
				{
					TArray<uint32>& TargetIndexes = BaseMorphIndexToTargetIndexList.FindOrAdd(MorphDelta.SourceIdx);
					TargetIndexes.AddUnique(TargetIndex);
				}
			}
		}
	}
	//Create the target morph target
	CreateLODMorphTarget(SkeletalMesh, InlineReductionDataParameter, SourceLOD, DestinationLOD, PerMorphTargetBaseIndexToMorphTargetDelta, BaseMorphIndexToTargetIndexList, TargetVertices, TargetMatchData);
}

void FLODUtilities::SimplifySkeletalMeshLOD(
	USkeletalMesh* SkeletalMesh,
	int32 DesiredLOD,
	const ITargetPlatform* TargetPlatform,
	bool bRestoreClothing, 
	TArray<FMorphTargetUpdateRequest>& OutMorphRequests,
	std::atomic<bool>* OutNeedsPackageDirtied
	)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLODUtilities::SimplifySkeletalMeshLOD);

	IMeshReductionModule& ReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionModule>("MeshReductionInterface");
	IMeshReduction* MeshReduction = ReductionModule.GetSkeletalMeshReductionInterface();
	if (!MeshReduction)
	{
		UE_ASSET_LOG(LogLODUtilities, Warning, SkeletalMesh, TEXT("Cannot reduce skeletalmesh LOD because there is no active reduction plugin."));
		return;
	}

	check (MeshReduction->IsSupported());


	if (DesiredLOD == 0
		&& SkeletalMesh->GetLODInfo(DesiredLOD) != nullptr
		&& SkeletalMesh->GetLODInfo(DesiredLOD)->bHasBeenSimplified
		&& !SkeletalMesh->GetImportedModel()->InlineReductionCacheDatas.IsValidIndex(0))
	{
		//The base LOD was reduce and there is no valid data, we cannot regenerate this lod it must be re-import before
		FFormatNamedArguments Args;
		Args.Add(TEXT("SkeletalMeshName"), FText::FromString(SkeletalMesh->GetName()));
		Args.Add(TEXT("LODIndex"), FText::AsNumber(DesiredLOD));
		FText Message = FText::Format(NSLOCTEXT("UnrealEd", "MeshSimp_GenerateLODCannotGenerateMissingData", "Cannot generate LOD {LODIndex} for skeletal mesh '{SkeletalMeshName}'. This LOD must be re-import to create the necessary data"), Args);

		if (FApp::IsUnattended() || !IsInGameThread())
		{
			UE_LOGF(LogLODUtilities, Warning, "%ls", *(Message.ToString()));
		}
		else
		{
			FMessageDialog::Open(EAppMsgType::Ok, Message);
		}
		return;
	}

	if (IsInGameThread())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("DesiredLOD"), DesiredLOD);
		Args.Add(TEXT("SkeletalMeshName"), FText::FromString(SkeletalMesh->GetName()));
		const FText StatusUpdate = FText::Format(NSLOCTEXT("UnrealEd", "MeshSimp_GeneratingLOD_F", "Generating LOD{DesiredLOD} for {SkeletalMeshName}..."), Args);
		GWarn->BeginSlowTask(StatusUpdate, true);
	}

	FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkeletalMesh);

	// Unbind DesiredLOD existing clothing assets before we simplify this LOD
	TArray<ClothingAssetUtils::FClothingAssetMeshBinding> ClothingBindings;
	if (bRestoreClothing && SkeletalMesh->GetImportedModel() && SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(DesiredLOD))
	{
		FLODUtilities::UnbindClothingAndBackup(SkeletalMesh, ClothingBindings, DesiredLOD);
	}

	FInlineReductionDataParameter InlineReductionDataParameter;

	if (SkeletalMesh->GetLODInfo(DesiredLOD) != nullptr)
	{
		FSkeletalMeshModel* SkeletalMeshResource = SkeletalMesh->GetImportedModel();
		FSkeletalMeshOptimizationSettings& Settings = SkeletalMesh->GetLODInfo(DesiredLOD)->ReductionSettings;

		//We must save the original reduction data, special case when we reduce inline we save even if its already simplified
		if (SkeletalMeshResource->LODModels.IsValidIndex(DesiredLOD) && (!SkeletalMesh->GetLODInfo(DesiredLOD)->bHasBeenSimplified || DesiredLOD == Settings.BaseLOD))
		{

			FSkeletalMeshLODModel& SrcModel = SkeletalMeshResource->LODModels[DesiredLOD];
			if (!SkeletalMeshResource->InlineReductionCacheDatas.IsValidIndex(DesiredLOD))
			{
				//We should not do that in a worker thread, the serialization of the SkeletalMeshResource is suppose to allocate the correct number of inline data caches
				//If the user add LOD in person editor, the simplification will be call in the game thread, see FLODUtilities::RegenerateLOD
				if (!ensure(IsInGameThread()))
				{
					UE_ASSET_LOG(LogLODUtilities, Error, SkeletalMesh, TEXT("FLODUtilities::SimplifySkeletalMeshLOD: InlineReductionCacheDatas was not added in the game thread."));
				}
				SkeletalMeshResource->InlineReductionCacheDatas.AddDefaulted((DesiredLOD + 1) - SkeletalMeshResource->InlineReductionCacheDatas.Num());
			}
			check(SkeletalMeshResource->InlineReductionCacheDatas.IsValidIndex(DesiredLOD));
			SkeletalMeshResource->InlineReductionCacheDatas[DesiredLOD].SetCacheGeometryInfo(SrcModel);

			InlineReductionDataParameter.InlineOriginalSrcMorphTargetData.Empty(SkeletalMesh->GetMorphTargets().Num());
			for (const UMorphTarget* MorphTarget : SkeletalMesh->GetMorphTargets())
			{
				// Iterate each original morph target source index to fill the NewMorphTargetDeltas array with the TargetMatchData.
				// If there's no morph data for the LOD, the returned array view is empty, and so nothing happens.
				TArray<FMorphTargetDelta>& MorphDeltasArray = InlineReductionDataParameter.InlineOriginalSrcMorphTargetData.FindOrAdd(MorphTarget->GetFullName());
				MorphDeltasArray.Append(MorphTarget->GetMorphTargetDeltas(DesiredLOD));
			}

			// Copy the original SkeletalMesh LODModel
			// Unbind clothing before saving the original data, we must not restore clothing to do inline reduction
			{
				TArray<ClothingAssetUtils::FClothingAssetMeshBinding> TemporaryRemoveClothingBindings;
				FLODUtilities::UnbindClothingAndBackup(SkeletalMesh, TemporaryRemoveClothingBindings, DesiredLOD);

				FSkeletalMeshLODModel::CopyStructure(&InlineReductionDataParameter.InlineOriginalSrcModel, &SrcModel);

				if (TemporaryRemoveClothingBindings.Num() > 0)
				{
					FLODUtilities::RestoreClothingFromBackup(SkeletalMesh, TemporaryRemoveClothingBindings, DesiredLOD);
				}
			}
			InlineReductionDataParameter.bIsDataValid = true;

			if (DesiredLOD == 0)
			{
				const UAssetImportData* AssetImportData = SkeletalMesh->GetAssetImportData();
				SkeletalMesh->GetLODInfo(DesiredLOD)->SourceImportFilename = AssetImportData ? AssetImportData->GetFirstFilename() : FString();
			}
		}
	}

	if (MeshReduction->ReduceSkeletalMesh(SkeletalMesh, DesiredLOD, TargetPlatform))
	{
		check(SkeletalMesh->GetLODNum() >= 1);

		//Manage morph target after the reduction. either apply to the reduce LOD or clear them all
		{
			FSkeletalMeshOptimizationSettings& ReductionSettings = SkeletalMesh->GetLODInfo(DesiredLOD)->ReductionSettings;
			//Apply morph to the new LOD. Force it if we reduce the base LOD, base LOD must apply the morph target
			if (ReductionSettings.bRemapMorphTargets)
			{
				OutMorphRequests.Emplace(SkeletalMesh, ReductionSettings.BaseLOD, DesiredLOD, MoveTemp(InlineReductionDataParameter));
			}
			else
			{
				OutMorphRequests.Emplace(SkeletalMesh, DesiredLOD);
			}

			// Update the vertex attribute information in the LOD info.
			UpdateLODInfoVertexAttributes(SkeletalMesh, ReductionSettings.BaseLOD, DesiredLOD, false);
		}

		if (IsInGameThread())
		{
			(void)SkeletalMesh->MarkPackageDirty();
		}
		else if(OutNeedsPackageDirtied)
		{
			(*OutNeedsPackageDirtied) = true;
		}
	}
	else
	{
		// Simplification failed! Warn the user.
		FFormatNamedArguments Args;
		Args.Add(TEXT("SkeletalMeshName"), FText::FromString(SkeletalMesh->GetName()));
		const FText Message = FText::Format(NSLOCTEXT("UnrealEd", "MeshSimp_GenerateLODFailed_F", "An error occurred while simplifying the geometry for mesh '{SkeletalMeshName}'.  Consider adjusting simplification parameters and re-simplifying the mesh."), Args);

		if (FApp::IsUnattended() || !IsInGameThread())
		{
			UE_LOGF(LogLODUtilities, Warning, "%ls", *(Message.ToString()));
		}
		else
		{
			FMessageDialog::Open(EAppMsgType::Ok, Message);
		}
	}

	//Put back the clothing for the DesiredLOD
	if (bRestoreClothing && ClothingBindings.Num() > 0 && SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(DesiredLOD))
	{
		FLODUtilities::RestoreClothingFromBackup(SkeletalMesh, ClothingBindings, DesiredLOD);
	}

	if (IsInGameThread())
	{
		GWarn->EndSlowTask();
	}
}

void FLODUtilities::SimplifySkeletalMeshLOD(FSkeletalMeshUpdateContext& UpdateContext, int32 DesiredLOD, const ITargetPlatform* TargetPlatform, bool bRestoreClothing /*= false*/, std::atomic<bool>* OutNeedsPackageDirtied/*= nullptr*/)
{
	USkeletalMesh* SkeletalMesh = UpdateContext.SkeletalMesh;
	IMeshReductionModule& ReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionModule>("MeshReductionInterface");
	IMeshReduction* MeshReduction = ReductionModule.GetSkeletalMeshReductionInterface();

	if (MeshReduction && MeshReduction->IsSupported() && SkeletalMesh)
	{
		TArray<FMorphTargetUpdateRequest> MorphTargetUpdateRequests;
		SimplifySkeletalMeshLOD(SkeletalMesh, DesiredLOD, TargetPlatform, bRestoreClothing, MorphTargetUpdateRequests, OutNeedsPackageDirtied);

		for (FMorphTargetUpdateRequest& Request: MorphTargetUpdateRequests)
		{
			Request.Run();
		}
		
		if (UpdateContext.OnLODChanged.IsBound())
		{
			//Notify calling system of change
			UpdateContext.OnLODChanged.ExecuteIfBound();
		}
	}
}

bool FLODUtilities::RestoreSkeletalMeshLODImportedData_DEPRECATED(USkeletalMesh* SkeletalMesh, int32 LodIndex)
{
	const bool bThisFunctionIsDeprecated = true;
	ensure(!bThisFunctionIsDeprecated);
	UE_ASSET_LOG(LogLODUtilities, Error, SkeletalMesh, TEXT("FLODUtilities::RestoreSkeletalMeshLODImportedData_DEPRECATED: This function is deprecated."));
	return false;
}

void FLODUtilities::RefreshLODChange(const USkeletalMesh* SkeletalMesh)
{
	for (FThreadSafeObjectIterator Iter(USkeletalMeshComponent::StaticClass()); Iter; ++Iter)
	{
		USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(*Iter);
		if  (SkeletalMeshComponent->GetSkeletalMeshAsset() == SkeletalMesh)
		{
			// it needs to recreate IF it already has been created
			if (SkeletalMeshComponent->IsRegistered())
			{
				SkeletalMeshComponent->UpdateLODStatus();
				SkeletalMeshComponent->MarkRenderStateDirty();
			}
		}
	}
}

bool ValidateAlternateSkeleton(const FSkeletalMeshImportData& ImportDataSrc, const FSkeletalMeshImportData& ImportDataDest, const FString& SkeletalMeshDestName, const int32 LODIndexDest)
{
	bool bIsunattended = GIsRunningUnattendedScript || FApp::IsUnattended();

	int32 BoneNumberDest = ImportDataDest.RefBonesBinary.Num();
	int32 BoneNumberSrc = ImportDataSrc.RefBonesBinary.Num();
	//We also want to report any missing bone, because skinning quality will be impacted if bones are missing
	TArray<FString> DestBonesNotUsedBySrc;
	TArray<FString> SrcBonesNotUsedByDest;
	for (int32 BoneIndexSrc = 0; BoneIndexSrc < BoneNumberSrc; ++BoneIndexSrc)
	{
		FString BoneNameSrc = ImportDataSrc.RefBonesBinary[BoneIndexSrc].Name;
		bool bFoundMatch = false;
		for (int32 BoneIndexDest = 0; BoneIndexDest < BoneNumberDest; ++BoneIndexDest)
		{
			if (ImportDataDest.RefBonesBinary[BoneIndexDest].Name.Equals(BoneNameSrc))
			{
				bFoundMatch = true;
				break;
			}
		}
		if (!bFoundMatch)
		{
			SrcBonesNotUsedByDest.Add(BoneNameSrc);
		}
	}

	for (int32 BoneIndexDest = 0; BoneIndexDest < BoneNumberDest; ++BoneIndexDest)
	{
		FString BoneNameDest = ImportDataDest.RefBonesBinary[BoneIndexDest].Name;
		bool bFound = false;
		for (int32 BoneIndexSrc = 0; BoneIndexSrc < BoneNumberSrc; ++BoneIndexSrc)
		{
			FString BoneNameSrc = ImportDataSrc.RefBonesBinary[BoneIndexSrc].Name;
			if (BoneNameDest.Equals(BoneNameSrc))
			{
				bFound = true;
				break;
			}
		}
		if (!bFound)
		{
			DestBonesNotUsedBySrc.Add(BoneNameDest);
		}
	}

	if (SrcBonesNotUsedByDest.Num() > 0)
	{
		//Let the user know
		if (!bIsunattended)
		{
			FString BoneList;
			for (FString& BoneName : SrcBonesNotUsedByDest)
			{
				BoneList += BoneName;
				BoneList += TEXT("\n");
			}

			FFormatNamedArguments Args;
			Args.Add(TEXT("SkeletalMeshName"), FText::FromString(SkeletalMeshDestName));
			Args.Add(TEXT("LODIndex"), FText::AsNumber(LODIndexDest));
			Args.Add(TEXT("BoneList"), FText::FromString(BoneList));
			FText Message = FText::Format(NSLOCTEXT("UnrealEd", "AlternateSkinningImport_SourceBoneNotUseByDestination", "Not all the alternate mesh bones are used by the LOD {LODIndex} when importing alternate weights for skeletal mesh '{SkeletalMeshName}'.\nBones List:\n{BoneList}"), Args);
			if(FMessageDialog::Open(EAppMsgType::OkCancel, Message) == EAppReturnType::Cancel)
			{
				return false;
			}
		}
		else
		{
			UE_LOGF(LogLODUtilities, Error, "Alternate skinning import: Not all the alternate mesh bones are used by the mesh.");
			return false;
		}
	}
	else if (DestBonesNotUsedBySrc.Num() > 0) //Do a else here since the DestBonesNotUsedBySrc is less prone to give a bad alternate influence result.
	{
		//Let the user know
		if (!bIsunattended)
		{
			FString BoneList;
			for (FString& BoneName : DestBonesNotUsedBySrc)
			{
				BoneList += BoneName;
				BoneList += TEXT("\n");
			}

			FFormatNamedArguments Args;
			Args.Add(TEXT("SkeletalMeshName"), FText::FromString(SkeletalMeshDestName));
			Args.Add(TEXT("LODIndex"), FText::AsNumber(LODIndexDest));
			Args.Add(TEXT("BoneList"), FText::FromString(BoneList));
			FText Message = FText::Format(NSLOCTEXT("UnrealEd", "AlternateSkinningImport_DestinationBoneNotUseBySource", "Not all the LOD {LODIndex} bones are used by the alternate mesh when importing alternate weights for skeletal mesh '{SkeletalMeshName}'.\nBones List:\n{BoneList}"), Args);
			if (FMessageDialog::Open(EAppMsgType::OkCancel, Message) == EAppReturnType::Cancel)
			{
				return false;
			}
		}
		else
		{
			UE_LOGF(LogLODUtilities, Display, "Alternate skinning import: Not all the mesh bones are used by the alternate mesh.");
			return false;
		}
	}

	return true;
}

/*
 * The remap use the name to find the corresponding bone index between the source and destination skeleton
 */
void FillRemapBoneIndexSrcToDest(const FSkeletalMeshImportData& ImportDataSrc, const FSkeletalMeshImportData& ImportDataDest, TMap<int32, int32>& RemapBoneIndexSrcToDest)
{
	RemapBoneIndexSrcToDest.Empty(ImportDataSrc.RefBonesBinary.Num());
	int32 BoneNumberDest = ImportDataDest.RefBonesBinary.Num();
	int32 BoneNumberSrc = ImportDataSrc.RefBonesBinary.Num();
	for (int32 BoneIndexSrc = 0; BoneIndexSrc < BoneNumberSrc; ++BoneIndexSrc)
	{
		FString BoneNameSrc = ImportDataSrc.RefBonesBinary[BoneIndexSrc].Name;
		for (int32 BoneIndexDest = 0; BoneIndexDest < BoneNumberDest; ++BoneIndexDest)
		{
			if (ImportDataDest.RefBonesBinary[BoneIndexDest].Name.Equals(BoneNameSrc))
			{
				RemapBoneIndexSrcToDest.Add(BoneIndexSrc, BoneIndexDest);
				break;
			}
		}
		if (!RemapBoneIndexSrcToDest.Contains(BoneIndexSrc))
		{
			RemapBoneIndexSrcToDest.Add(BoneIndexSrc, INDEX_NONE);
		}
	}
}

namespace VertexMatchNameSpace
{
	struct FVertexMatchResult
	{
		TArray<uint32> VertexIndexes;
		TArray<float> Ratios;
	};
}

struct FTriangleOctreeSemantics
{
	// When a leaf gets more than this number of elements, it will split itself into a node with multiple child leaves
	enum { MaxElementsPerLeaf = 10 };

	// This is used for incremental updates.  When removing a polygon, larger values will cause leaves to be removed and collapsed into a parent node.
	enum { MinInclusiveElementsPerNode = 5 };

	// How deep the tree can go.
	enum { MaxNodeDepth = 20 };


	typedef TInlineAllocator<MaxElementsPerLeaf> ElementAllocator;

	FORCEINLINE static FBoxCenterAndExtent GetBoundingBox(const FTriangleElement& Element)
	{
		return Element.PositionBound;
	}

	FORCEINLINE static bool AreElementsEqual(const FTriangleElement& A, const FTriangleElement& B)
	{
		return (A.TriangleIndex == B.TriangleIndex);
	}

	FORCEINLINE static void SetElementId(const FTriangleElement& Element, FOctreeElementId2 OctreeElementID)
	{
	}
};

typedef TOctree2<FTriangleElement, FTriangleOctreeSemantics> TTriangleElementOctree;

void MatchVertexIndexUsingPosition(
	const FSkeletalMeshImportData& ImportDataDest
	, const FSkeletalMeshImportData& ImportDataSrc
	, TSortedMap<uint32, VertexMatchNameSpace::FVertexMatchResult>& VertexIndexSrcToVertexIndexDestMatches
	, const TArray<uint32>& VertexIndexToMatchWithPositions
	, bool& bNoMatchMsgDone
	, const TCHAR* DebugContext)
{
	if (VertexIndexToMatchWithPositions.Num() <= 0)
	{
		return;
	}
	int32 FaceNumberDest = ImportDataDest.Faces.Num();

	//Setup the Position Octree with the destination faces so we can match the source vertex index
	TArray<FTriangleElement> TrianglesDest;
	FBox BaseMeshPositionBound(EForceInit::ForceInit);

	for (int32 FaceIndexDest = 0; FaceIndexDest < FaceNumberDest; ++FaceIndexDest)
	{
		const SkeletalMeshImportData::FTriangle& Triangle = ImportDataDest.Faces[FaceIndexDest];
		FTriangleElement TriangleElement;
		TriangleElement.UVsBound.Init();

		FBox TrianglePositionBound;
		TrianglePositionBound.Init();

		for (int32 Corner = 0; Corner < 3; ++Corner)
		{
			const uint32 WedgeIndexDest = Triangle.WedgeIndex[Corner];
			const uint32 VertexIndexDest = ImportDataDest.Wedges[WedgeIndexDest].VertexIndex;
			TriangleElement.Indexes.Add(WedgeIndexDest);
			FSoftSkinVertex SoftSkinVertex;
			SoftSkinVertex.Position = ImportDataDest.Points[VertexIndexDest];
			SoftSkinVertex.UVs[0] = ImportDataDest.Wedges[WedgeIndexDest].UVs[0];
			TriangleElement.Vertices.Add(SoftSkinVertex);
			TriangleElement.UVsBound += FVector2D(SoftSkinVertex.UVs[0]);
			TrianglePositionBound += (FVector)SoftSkinVertex.Position;
			BaseMeshPositionBound += (FVector)SoftSkinVertex.Position;
		}
		BaseMeshPositionBound += TrianglePositionBound;
		TriangleElement.PositionBound = FBoxCenterAndExtent(TrianglePositionBound);
		TriangleElement.TriangleIndex = FaceIndexDest;
		TrianglesDest.Add(TriangleElement);
	}

	TTriangleElementOctree OcTree(BaseMeshPositionBound.GetCenter(), BaseMeshPositionBound.GetExtent().Size());
	for (FTriangleElement& TriangleElement : TrianglesDest)
	{
		OcTree.AddElement(TriangleElement);
	}

	//Retrieve all triangles that are close to our point, start at 0.25% of OcTree extend
	double DistanceThreshold = BaseMeshPositionBound.GetExtent().Size()*0.0025;

	//Find a match triangle for every target vertices
	TArray<FTriangleElement> OcTreeTriangleResults;
	OcTreeTriangleResults.Reserve(TrianglesDest.Num() / 50); //Reserve 2% to speed up the query

	//This lambda store a source vertex index -> source wedge index destination triangle.
	//It use a barycentric function to determine the impact on the 3 corner of the triangle.
	auto AddMatchTriangle = [&ImportDataDest, &TrianglesDest, &VertexIndexSrcToVertexIndexDestMatches, DebugContext](const FTriangleElement& BestTriangle, const FVector3f& Position, const uint32 VertexIndexSrc)
	{
		//Found the surface area of the 3 barycentric triangles from the UVs
		FVector3f BarycentricWeight;
		BarycentricWeight = GetBaryCentric(Position, BestTriangle.Vertices[0].Position, BestTriangle.Vertices[1].Position, BestTriangle.Vertices[2].Position, DebugContext);
		//Fill the match
		VertexMatchNameSpace::FVertexMatchResult& VertexMatchDest = VertexIndexSrcToVertexIndexDestMatches.FindOrAdd(VertexIndexSrc);
		for (int32 CornerIndex = 0; CornerIndex < 3; ++CornerIndex)
		{
			int32 VertexIndexDest = ImportDataDest.Wedges[BestTriangle.Indexes[CornerIndex]].VertexIndex;
			float Ratio = BarycentricWeight[CornerIndex];
			int32 FindIndex = INDEX_NONE;
			if (!VertexMatchDest.VertexIndexes.Find(VertexIndexDest, FindIndex))
			{
				VertexMatchDest.VertexIndexes.Add(VertexIndexDest);
				VertexMatchDest.Ratios.Add(Ratio);
			}
			else
			{
				check(VertexMatchDest.Ratios.IsValidIndex(FindIndex));
				VertexMatchDest.Ratios[FindIndex] = FMath::Max(VertexMatchDest.Ratios[FindIndex], Ratio);
			}
		}
	};

	for (int32 VertexIndexSrc : VertexIndexToMatchWithPositions)
	{
		FVector3f PositionSrc = ImportDataSrc.Points[VertexIndexSrc];
		OcTreeTriangleResults.Reset();

		//Use the OcTree to find closest triangle
		FVector Extent(DistanceThreshold, DistanceThreshold, DistanceThreshold);
		FBoxCenterAndExtent CurBox((FVector)PositionSrc, Extent);
		
		while (OcTreeTriangleResults.Num() <= 0)
		{
			OcTree.FindElementsWithBoundsTest(CurBox, [&OcTreeTriangleResults](const FTriangleElement& Element)
			{
				// Add all of the elements in the current node to the list of points to consider for closest point calculations
				OcTreeTriangleResults.Add(Element);
			});

			//Increase the extend so we try to found in a larger area
			Extent *= 2;
			if (Extent.SizeSquared() >= BaseMeshPositionBound.GetSize().SizeSquared())
			{
				//Extend must not be bigger then the whole mesh, its acceptable to have error at this point
				break;
			}
			CurBox = FBox((FVector)PositionSrc - Extent, (FVector)PositionSrc + Extent);
		}

		//Get the 3D distance between a point and a destination triangle
		auto GetDistanceSrcPointToDestTriangle = [&TrianglesDest, &PositionSrc](const uint32 DestTriangleIndex)->double
		{
			FTriangleElement& CandidateTriangle = TrianglesDest[DestTriangleIndex];
			return FVector::DistSquared(FMath::ClosestPointOnTriangleToPoint((FVector)PositionSrc, (FVector)CandidateTriangle.Vertices[0].Position, (FVector)CandidateTriangle.Vertices[1].Position, (FVector)CandidateTriangle.Vertices[2].Position), (FVector)PositionSrc);
		};

		//Brute force finding of closest triangle using 3D position
		auto FailSafeUnmatchVertex = [&GetDistanceSrcPointToDestTriangle, &OcTreeTriangleResults](uint32 &OutIndexMatch)->bool
		{
			bool bFoundMatch = false;
			double ClosestTriangleDistSquared = std::numeric_limits<double>::max();
			for (const FTriangleElement& MatchTriangle : OcTreeTriangleResults)
			{
				const int32 MatchTriangleIndex = MatchTriangle.TriangleIndex;
				const double TriangleDistSquared = GetDistanceSrcPointToDestTriangle(MatchTriangleIndex);
				if (TriangleDistSquared < ClosestTriangleDistSquared)
				{
					ClosestTriangleDistSquared = TriangleDistSquared;
					OutIndexMatch = MatchTriangleIndex;
					bFoundMatch = true;
				}
			}
			return bFoundMatch;
		};

		//Find all Triangles that contain the Target UV
		if (OcTreeTriangleResults.Num() > 0)
		{
			TArray<uint32> MatchTriangleIndexes;
			uint32 FoundIndexMatch = INDEX_NONE;
			if (!FindTrianglePositionMatch((FVector)PositionSrc, TrianglesDest, OcTreeTriangleResults, MatchTriangleIndexes))
			{
				//There is no Position match possible, use brute force fail safe
				if (!FailSafeUnmatchVertex(FoundIndexMatch))
				{
					//We should always have a match
					if (!bNoMatchMsgDone)
					{
						UE_LOGF(LogLODUtilities, Warning, "Alternate skinning import: Cannot find a triangle from the destination LOD that contain a vertex UV in the imported alternate skinning LOD mesh. Alternate skinning quality will be lower.");
						bNoMatchMsgDone = true;
					}
					continue;
				}
			}
			double ClosestTriangleDistSquared = std::numeric_limits<double>::max();
			if (MatchTriangleIndexes.Num() == 1)
			{
				//One match, this mean no mirror UVs simply take the single match
				FoundIndexMatch = MatchTriangleIndexes[0];
				ClosestTriangleDistSquared = GetDistanceSrcPointToDestTriangle(FoundIndexMatch);
			}
			else
			{
				//Geometry can use mirror so the UVs are not unique. Use the closest match triangle to the point to find the best match
				for (uint32 MatchTriangleIndex : MatchTriangleIndexes)
				{
					double TriangleDistSquared = GetDistanceSrcPointToDestTriangle(MatchTriangleIndex);
					if (TriangleDistSquared < ClosestTriangleDistSquared)
					{
						ClosestTriangleDistSquared = TriangleDistSquared;
						FoundIndexMatch = MatchTriangleIndex;
					}
				}
			}

			//FAIL SAFE, make sure we have a match that make sense
			//Use the mesh geometry bound extent (1% of it) to validate we are close enough.
			if (ClosestTriangleDistSquared > BaseMeshPositionBound.GetExtent().SizeSquared()*0.01f)
			{
				//Executing fail safe, if the UVs are too much off because of the reduction, use the closest distance to polygons to find the match
				//This path is not optimize and should not happen often.
				FailSafeUnmatchVertex(FoundIndexMatch);
			}

			//We should always have a valid match at this point
			check(TrianglesDest.IsValidIndex(FoundIndexMatch));
			AddMatchTriangle(TrianglesDest[FoundIndexMatch], PositionSrc, VertexIndexSrc);
		}
		else
		{
			if (!bNoMatchMsgDone)
			{
				UE_LOGF(LogLODUtilities, Warning, "Alternate skinning import: Cannot find a triangle from the destination LOD that contain a vertex UV in the imported alternate skinning LOD mesh. Alternate skinning quality will be lower.");
				bNoMatchMsgDone = true;
			}
		}
	}
}

bool FLODUtilities::UpdateAlternateSkinWeights(USkeletalMesh* SkeletalMeshDest, const FName& ProfileNameDest, int32 LODIndexDest, const IMeshUtilities::MeshBuildOptions& Options)
{
	//Grab all the destination structure
	check(SkeletalMeshDest);
	check(SkeletalMeshDest->GetImportedModel());
	check(SkeletalMeshDest->GetImportedModel()->LODModels.IsValidIndex(LODIndexDest));
	FSkeletalMeshLODModel& LODModelDest = SkeletalMeshDest->GetImportedModel()->LODModels[LODIndexDest];
	if (!SkeletalMeshDest->HasMeshDescription(LODIndexDest))
	{
		UE_LOGF(LogLODUtilities, Error, "Failed to import Skin Weight Profile as the target skeletal mesh (%ls) requires reimporting first.", *SkeletalMeshDest->GetName());
		//Very old asset will not have this data, we cannot add alternate until the asset is reimported
		return false;
	}
	
	FSkeletalMeshImportData ImportDataDest;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SkeletalMeshDest->LoadLODImportedData(LODIndexDest, ImportDataDest);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	return UpdateAlternateSkinWeights(
		LODModelDest,
		ImportDataDest,
		SkeletalMeshDest,
		SkeletalMeshDest->GetRefSkeleton(),
		ProfileNameDest,
		LODIndexDest,
		Options);
}

bool FLODUtilities::UpdateAlternateSkinWeights(
	USkeletalMesh* SkeletalMeshDest,
	const FName& ProfileNameDest,
	int32 LODIndexDest,
	FOverlappingThresholds OverlappingThresholds,
	bool ShouldImportNormals,
	bool ShouldImportTangents,
	bool bUseMikkTSpace,
	bool bComputeWeightedNormals)
{
	IMeshUtilities::MeshBuildOptions Options;
	Options.OverlappingThresholds = OverlappingThresholds;
	Options.bComputeNormals = !ShouldImportNormals;
	Options.bComputeTangents = !ShouldImportTangents;
	Options.bUseMikkTSpace = bUseMikkTSpace;
	Options.bComputeWeightedNormals = bComputeWeightedNormals;

	return UpdateAlternateSkinWeights(SkeletalMeshDest, ProfileNameDest, LODIndexDest, Options);
}


static bool IsSameImportMeshStructure(const FSkeletalMeshImportData& MeshA, const FSkeletalMeshImportData& MeshB)
{
	// This equivalency testing above will suffice and is only for sanity, since both of the import data objects
	// passed in should have been generated from the same FMeshDescription (see FSkeletalMeshImportData::GetMeshDescription)
	return MeshA.Points.Num() == MeshB.Points.Num() && 
	       MeshA.Wedges.Num() == MeshB.Wedges.Num() &&
		   MeshA.Faces.Num() == MeshB.Faces.Num() &&
	       MeshA.RefBonesBinary.Num() == MeshB.RefBonesBinary.Num();
}


bool FLODUtilities::UpdateAlternateSkinWeights(
	FSkeletalMeshLODModel& LODModelDest,
	FSkeletalMeshImportData& ImportDataDest,
	USkeletalMesh* SkeletalMeshDest,
	const FReferenceSkeleton& RefSkeleton,
	const FName& ProfileNameDest,
	int32 LODIndexDest,
	const IMeshUtilities::MeshBuildOptions& Options)
{
	int32 ProfileIndex = 0;
	if (!ImportDataDest.AlternateInfluenceProfileNames.Find(ProfileNameDest.ToString(), ProfileIndex))
	{
		UE_LOGFMT(LogLODUtilities, Display, "Can't update alternative profile {ProfileName} on {SkeletalMesh} at LOD {0}. Profile doesn't exist.",
			ProfileNameDest, SkeletalMeshDest->GetName(), LODIndexDest);
		return false;
	}

	check(ImportDataDest.AlternateInfluences.IsValidIndex(ProfileIndex));
	
	//The data must be there and must be verified before getting here
	const FSkeletalMeshImportData& ImportDataSrc = ImportDataDest.AlternateInfluences[ProfileIndex];

	// Double-check the mesh has the same structure.
	if (!IsSameImportMeshStructure(ImportDataSrc, ImportDataDest))
	{
		UE_LOGFMT(LogLODUtilities, Warning, "Can't update alternative profile {ProfileName} on {SkeletalMesh} at LOD {0}. Profile source mesh and the base mesh are different.",
			ProfileNameDest, SkeletalMeshDest->GetName(), LODIndexDest);
		return false;
	}

	//Sort and normalize weights for alternate influences
	TArray<SkeletalMeshImportData::FRawBoneInfluence> AlternateInfluences{ImportDataSrc.Influences};
	
	ProcessImportMeshInfluences(ImportDataDest.Wedges.Num(), AlternateInfluences, SkeletalMeshDest->GetPathName());

	//Store the remapped influence into the profile, the function SkeletalMeshTools::ChunkSkinnedVertices will use all profiles including this one to chunk the sections
	FImportedSkinWeightProfileData& ImportedProfileData = LODModelDest.SkinWeightProfiles.Add(ProfileNameDest);
	ImportedProfileData.SourceModelInfluences.Empty(AlternateInfluences.Num());
	for (int32 InfluenceIndex = 0; InfluenceIndex < AlternateInfluences.Num(); ++InfluenceIndex)
	{
		const SkeletalMeshImportData::FRawBoneInfluence& RawInfluence = AlternateInfluences[InfluenceIndex];
		SkeletalMeshImportData::FVertInfluence LODAlternateInfluence;
		LODAlternateInfluence.BoneIndex = static_cast<FBoneIndexType>(RawInfluence.BoneIndex);
		LODAlternateInfluence.VertIndex = RawInfluence.VertexIndex;
		LODAlternateInfluence.Weight = RawInfluence.Weight;
		ImportedProfileData.SourceModelInfluences.Add(LODAlternateInfluence);
	}

	//
	//////////////////////////////////////////////////////////////////////////

	//Prepare the build data to rebuild the asset with the alternate influences
	//The chunking can be different when we have alternate influences
	//Grab the build data from ImportDataDest
	TArray<FVector3f> LODPointsDest;
	TArray<SkeletalMeshImportData::FMeshWedge> LODWedgesDest;
	TArray<SkeletalMeshImportData::FMeshFace> LODFacesDest;
	TArray<SkeletalMeshImportData::FVertInfluence> LODInfluencesDest;
	TArray<int32> LODPointToRawMapDest;
	ImportDataDest.CopyLODImportData(LODPointsDest, LODWedgesDest, LODFacesDest, LODInfluencesDest, LODPointToRawMapDest);

	//Set the options with the current asset build options
	IMeshUtilities::MeshBuildOptions BuildOptions = Options;
	BuildOptions.bComputeNormals = Options.bComputeNormals || !ImportDataDest.bHasNormals;
	BuildOptions.bComputeTangents = Options.bComputeTangents || !ImportDataDest.bHasTangents;
	BuildOptions.bUseMikkTSpace = (Options.bUseMikkTSpace) && (Options.bComputeNormals || Options.bComputeTangents);

	//Build the skeletal mesh asset
	IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");
	TArray<FText> WarningMessages;
	TArray<FName> WarningNames;

	//BaseLOD need to make sure the source data fit with the skeletalmesh materials array before using meshutilities.BuildSkeletalMesh
	AdjustImportDataFaceMaterialIndex(SkeletalMeshDest->GetMaterials(), ImportDataDest.Materials, LODFacesDest, LODIndexDest);

	//Build the destination mesh with the Alternate influences, so the chunking is done properly.
	bool bBuildSuccess = MeshUtilities.BuildSkeletalMesh(LODModelDest, SkeletalMeshDest->GetName(), RefSkeleton, LODInfluencesDest, LODWedgesDest, LODFacesDest, LODPointsDest, LODPointToRawMapDest, BuildOptions, &WarningMessages, &WarningNames);
	//Re-Apply the user section changes, the UserSectionsData is map to original section and should match the builded LODModel
	LODModelDest.SyncronizeUserSectionsDataArray();

	RegenerateAllImportSkinWeightProfileData(LODModelDest, Options.BoneInfluenceLimit, Options.TargetPlatform);
	
	return bBuildSuccess;
}


bool FLODUtilities::UpdateAlternateSkinWeights(
	FSkeletalMeshLODModel& LODModelDest,
	FSkeletalMeshImportData& ImportDataDest,
	USkeletalMesh* SkeletalMeshDest,
	const FReferenceSkeleton& RefSkeleton,
	const FName& ProfileNameDest,
	int32 LODIndexDest,
	FOverlappingThresholds OverlappingThresholds,
	bool ShouldImportNormals,
	bool ShouldImportTangents,
	bool bUseMikkTSpace,
	bool bComputeWeightedNormals)
{
	IMeshUtilities::MeshBuildOptions Options;
	Options.OverlappingThresholds = OverlappingThresholds;
	Options.bComputeNormals = !ShouldImportNormals;
	Options.bComputeTangents = !ShouldImportTangents;
	Options.bUseMikkTSpace = bUseMikkTSpace;
	Options.bComputeWeightedNormals = bComputeWeightedNormals;

	return UpdateAlternateSkinWeights(LODModelDest, ImportDataDest, SkeletalMeshDest, RefSkeleton, ProfileNameDest, LODIndexDest, Options);
}

bool FLODUtilities::UpdateAlternateSkinWeights(
	USkeletalMesh* SkeletalMeshDest,
	const FName& ProfileNameDest,
	USkeletalMesh* SkeletalMeshSrc,
	int32 LODIndexDest,
	int32 LODIndexSrc,
	const IMeshUtilities::MeshBuildOptions& Options)
{
	//Grab all the destination structure
	check(SkeletalMeshDest);
	check(SkeletalMeshDest->GetImportedModel());
	check(SkeletalMeshDest->GetImportedModel()->LODModels.IsValidIndex(LODIndexDest));
	FSkeletalMeshLODModel& LODModelDest = SkeletalMeshDest->GetImportedModel()->LODModels[LODIndexDest];

	if (!SkeletalMeshDest->HasMeshDescription(LODIndexDest))
	{
		UE_LOGF(LogLODUtilities, Error, "Failed to import Skin Weight Profile as the target skeletal mesh (%ls) requires reimporting first.", SkeletalMeshDest ? *SkeletalMeshDest->GetName() : TEXT("NULL"));
		//Very old asset will not have this data, we cannot add alternate until the asset is reimported
		return false;
	}
	FSkeletalMeshImportData ImportDataDest;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SkeletalMeshDest->LoadLODImportedData(LODIndexDest, ImportDataDest);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	int32 PointNumberDest = ImportDataDest.Points.Num();
	int32 VertexNumberDest = ImportDataDest.Points.Num();

	//Grab all the source structure
	check(SkeletalMeshSrc);

	//The source model is a fresh import and the data need to be there
	check(SkeletalMeshSrc->HasMeshDescription(LODIndexSrc));
	FSkeletalMeshImportData ImportDataSrc;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SkeletalMeshSrc->LoadLODImportedData(LODIndexSrc, ImportDataSrc);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	//Remove all unnecessary array data from the structure (this will save a lot of memory)
	ImportDataSrc.KeepAlternateSkinningBuildDataOnly();

	FString SkeletalMeshDestName = SkeletalMeshDest->GetName();
	if (ImportDataSrc.Points.Num() != PointNumberDest)
	{
		UE_LOGF(LogLODUtilities, Error, "Asset %ls failed to import Skin Weight Profile as the incomming alternate influence model vertex number is different. LOD model vertex count: %d Alternate model vertex count: %d", *SkeletalMeshDestName, PointNumberDest, ImportDataSrc.Points.Num());
		return false;
	}

	if (!ValidateAlternateSkeleton(ImportDataSrc, ImportDataDest, SkeletalMeshDestName, LODIndexDest))
	{
		//Log are print in the validate function
		return false;
	}

	//Replace the data into the destination bulk data and save it
	int32 ProfileIndex = 0;
	if (ImportDataDest.AlternateInfluenceProfileNames.Find(ProfileNameDest.ToString(), ProfileIndex))
	{
		ImportDataDest.AlternateInfluenceProfileNames.RemoveAt(ProfileIndex);
		ImportDataDest.AlternateInfluences.RemoveAt(ProfileIndex);
	}
	ImportDataDest.AlternateInfluenceProfileNames.Add(ProfileNameDest.ToString());
	ImportDataDest.AlternateInfluences.Add(ImportDataSrc);

	//Resave the bulk data with the new or refreshed data
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SkeletalMeshDest->SaveLODImportedData(LODIndexDest, ImportDataDest);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return true;
}

bool FLODUtilities::UpdateAlternateSkinWeights(
	USkeletalMesh* SkeletalMeshDest,
	const FName& ProfileNameDest,
	USkeletalMesh* SkeletalMeshSrc,
	int32 LODIndexDest,
	int32 LODIndexSrc,
	FOverlappingThresholds OverlappingThresholds,
	bool ShouldImportNormals,
	bool ShouldImportTangents,
	bool bUseMikkTSpace,
	bool bComputeWeightedNormals)
{
	IMeshUtilities::MeshBuildOptions Options;
	Options.OverlappingThresholds = OverlappingThresholds;
	Options.bComputeNormals = !ShouldImportNormals;
	Options.bComputeTangents = !ShouldImportTangents;
	Options.bUseMikkTSpace = bUseMikkTSpace;
	Options.bComputeWeightedNormals = bComputeWeightedNormals;

	return UpdateAlternateSkinWeights(SkeletalMeshDest, ProfileNameDest, SkeletalMeshSrc, LODIndexDest, LODIndexSrc, Options);
}

void FLODUtilities::GenerateImportedSkinWeightProfileData(
	FSkeletalMeshLODModel& LODModelDest,
	FImportedSkinWeightProfileData& ImportedProfileData,
	int32 BoneInfluenceLimit,
	const ITargetPlatform* TargetPlatform)
{
	using namespace UE::AnimationCore;
	
	//Add the override buffer with the alternate influence data
	TArray<FSoftSkinVertex> DestinationSoftVertices;
	LODModelDest.GetVertices(DestinationSoftVertices);
	//Get the SkinWeights buffer allocated before filling it
	TArray<FRawSkinWeight>& SkinWeights = ImportedProfileData.SkinWeights;
	SkinWeights.Empty(DestinationSoftVertices.Num());

	const int32 MaxBoneInfluencesFromProjectSettings = FGPUBaseSkinVertexFactory::UseUnlimitedBoneInfluences(MAX_TOTAL_INFLUENCES, TargetPlatform) ? MAX_TOTAL_INFLUENCES : EXTRA_BONE_INFLUENCES;
	const int32 MaxBoneInfluencesFromAsset = FGPUBaseSkinVertexFactory::GetBoneInfluenceLimitForAsset(BoneInfluenceLimit, TargetPlatform);

	//Get the maximum allow bone influence, so we can cut lowest weight properly and get the same result has the sk build
	const int32 MaxInfluenceCount = FMath::Min(MaxBoneInfluencesFromProjectSettings, MaxBoneInfluencesFromAsset);

	int32 MaxNumInfluences = 0;

	TMultiMap<int32, const SkeletalMeshImportData::FVertInfluence*> VertexToInfluenceMap;
	for (const SkeletalMeshImportData::FVertInfluence& VertInfluence : ImportedProfileData.SourceModelInfluences)
	{
		VertexToInfluenceMap.Add(VertInfluence.VertIndex, &VertInfluence);
	}
	TMap<FBoneIndexType, float> WeightForBone;
	TArray<const SkeletalMeshImportData::FVertInfluence*> FoundInfluences;

	// cf. FCompareVertexIndex (should be factorized for next versions) 
	auto VertInfluencePredicate = [](const SkeletalMeshImportData::FVertInfluence* A, const SkeletalMeshImportData::FVertInfluence* B) 
	{
		if (A->VertIndex > B->VertIndex) return false;
		if (A->VertIndex < B->VertIndex) return true;
		if (A->Weight < B->Weight) return false;
		if (A->Weight > B->Weight) return true;
		if (A->BoneIndex > B->BoneIndex) return false;
		if (A->BoneIndex < B->BoneIndex) return true;
		return  false;
	};
	
	for (int32 VertexInstanceIndex = 0; VertexInstanceIndex < DestinationSoftVertices.Num(); ++VertexInstanceIndex)
	{
		int32 SectionIndex = INDEX_NONE;
		int32 OutVertexIndexGarb = INDEX_NONE;
		LODModelDest.GetSectionFromVertexIndex(VertexInstanceIndex, SectionIndex, OutVertexIndexGarb);
		if (!LODModelDest.Sections.IsValidIndex(SectionIndex))
		{
			continue;
		}
		
		FSkelMeshSection& Section = LODModelDest.Sections[SectionIndex];
		const TArray<FBoneIndexType>& SectionBoneMap = Section.BoneMap;
		const int32 VertexIndex = LODModelDest.GetRawPointIndices()[VertexInstanceIndex];
		FRawSkinWeight& SkinWeight = SkinWeights.AddDefaulted_GetRef();

		// get the bone weights for that vertex and sort them (mostly by weights here as VertIndex are similar)
		FoundInfluences.Reset();
		VertexToInfluenceMap.MultiFind(VertexIndex, FoundInfluences);
		Algo::Sort(FoundInfluences, VertInfluencePredicate);

		WeightForBone.Reset();
		for (const SkeletalMeshImportData::FVertInfluence* VertInfluence : FoundInfluences)
		{
			//Use the section bone map to remap the bone index
			int32 BoneMapIndex = INDEX_NONE;
			SectionBoneMap.Find(VertInfluence->BoneIndex, BoneMapIndex);
			if (BoneMapIndex == INDEX_NONE)
			{
				//Map to root of the section
				BoneMapIndex = 0;
			}
			WeightForBone.Add(static_cast<FBoneIndexType>(BoneMapIndex), VertInfluence->Weight);
		}

		//Add the prepared alternate influences for this skin vertex (cf. MeshUtilities::GenerateSkeletalRenderMesh)
		int32 InfluenceBoneCount = FMath::Min<uint32>(WeightForBone.Num(), MaxBoneInfluencesFromAsset);
		if (InfluenceBoneCount > EXTRA_BONE_INFLUENCES && !FGPUBaseSkinVertexFactory::UseUnlimitedBoneInfluences(InfluenceBoneCount, TargetPlatform))
		{
			InfluenceBoneCount = EXTRA_BONE_INFLUENCES;
		}
		
		FBoneIndexType InfluenceBones[MAX_TOTAL_INFLUENCES];
		float InfluenceWeights[MAX_TOTAL_INFLUENCES];

		int32 ActualInfluenceCount = 0;
		for (const TTuple<FBoneIndexType, float>& BoneAndWeight : WeightForBone)
		{
			if (ActualInfluenceCount < InfluenceBoneCount)
			{
				InfluenceBones[ActualInfluenceCount] = BoneAndWeight.Key;
				InfluenceWeights[ActualInfluenceCount] = BoneAndWeight.Value;
				ActualInfluenceCount++;
			}
		}

		const FBoneWeights BoneWeights = FBoneWeights::Create(InfluenceBones, InfluenceWeights, ActualInfluenceCount);
		FMemory::Memzero(SkinWeight.InfluenceBones);
		FMemory::Memzero(SkinWeight.InfluenceWeights);
	
		if (BoneWeights.Num() == 0)
		{
			SkinWeight.InfluenceWeights[0] = std::numeric_limits<uint16>::max();
		}
		else
		{
			int32 Index = 0;
			for (const FBoneWeight& BoneWeight: BoneWeights)
			{
				SkinWeight.InfluenceBones[Index] = BoneWeight.GetBoneIndex();
				SkinWeight.InfluenceWeights[Index] = BoneWeight.GetRawWeight();
				Index++;
			}
		}

		//Adjust section influence count if the alternate influence bone count is greater
		if (ActualInfluenceCount > MaxNumInfluences)
		{
			MaxNumInfluences = ActualInfluenceCount;
			if (MaxNumInfluences > Section.GetMaxBoneInfluences())
			{
				Section.MaxBoneInfluences = MaxNumInfluences;
			}
		}
		check(MaxNumInfluences <= MaxInfluenceCount);
	}
}

void FLODUtilities::RegenerateAllImportSkinWeightProfileData(FSkeletalMeshLODModel& LODModelDest, int32 BoneInfluenceLimit, const ITargetPlatform* TargetPlatform)
{
	for (TPair<FName, FImportedSkinWeightProfileData>& ProfilePair : LODModelDest.SkinWeightProfiles)
	{
		GenerateImportedSkinWeightProfileData(LODModelDest, ProfilePair.Value, BoneInfluenceLimit, TargetPlatform);
	}
}



bool FLODUtilities::UpdateLODInfoVertexAttributes(
	USkeletalMesh *InSkeletalMesh,
	int32 InSourceLODIndex, 
	int32 InTargetLODIndex, 
	bool bInCopyAttributeValues
	)
{
	if (!InSkeletalMesh->HasMeshDescription(InSourceLODIndex))
	{
		return false;
	}

	FSkeletalMeshLODModel& TargetLODModel = InSkeletalMesh->GetImportedModel()->LODModels[InTargetLODIndex];
	
	const TArray<FSkeletalMeshVertexAttributeInfo>& SkelMeshAttributeInfos = InSkeletalMesh->GetLODInfo(InTargetLODIndex)->VertexAttributes; 

	// Retain any existing attribute infos and match based on names.
	TMap<FName, const FSkeletalMeshVertexAttributeInfo*> ExistingAttributeInfos;
	for (const FSkeletalMeshVertexAttributeInfo& AttributeInfo: SkelMeshAttributeInfos)
	{
		ExistingAttributeInfos.Add(AttributeInfo.Name, &AttributeInfo);
	}
	
	const FMeshDescription* MeshDescription = InSkeletalMesh->GetMeshDescription(InSourceLODIndex);

	// NOTE: There's a current limitation that we only support single-channel attributes for rendering.
	TMap<FName, TVertexAttributesConstRef<float>> SourceAttributes; 
	MeshDescription->VertexAttributes().ForEachByType<float>([&SourceAttributes](const FName InAttributeName, TVertexAttributesConstRef<float> InAttributeRef)
	{
		if (!FSkeletalMeshAttributes::IsReservedAttributeName(InAttributeName))
		{
			SourceAttributes.Add(InAttributeName, InAttributeRef);
		}
	});
 
	// If we're not copying the values, leave the existing data in place.
	if (bInCopyAttributeValues)
	{
		TargetLODModel.VertexAttributes.Reset();
	}
	
	TArray<UE::Tasks::FTask> ConversionTasks;
	
	for (const TPair<FName, TVertexAttributesConstRef<float>>& SourceAttributeInfo: SourceAttributes)
	{
		const TVertexAttributesConstRef<float> SourceAttribute = SourceAttributeInfo.Value;
		const FName AttributeName(SourceAttributeInfo.Key);

		// Did this definition already exist? Try to retain as much of the existing information as possible.
		const FSkeletalMeshVertexAttributeInfo* AttributeInfo = nullptr;
		
		if (ExistingAttributeInfos.Contains(AttributeName))
		{
			AttributeInfo = ExistingAttributeInfos[AttributeName];
		}

		if (AttributeInfo && AttributeInfo->IsEnabledForRender() && bInCopyAttributeValues)
		{
			FSkeletalMeshModelVertexAttribute& ModelAttribute = TargetLODModel.VertexAttributes.FindOrAdd(AttributeName);

			ModelAttribute.DataType = AttributeInfo->DataType;
			ModelAttribute.ComponentCount = 1;

			if (InTargetLODIndex == InSourceLODIndex)
			{
				ConversionTasks.Add(
					UE::Tasks::Launch(UE_SOURCE_LOCATION, [&TargetLODModel, AttributeName, SourceAttribute]()
					{
						FSkeletalMeshModelVertexAttribute& ModelAttribute = TargetLODModel.VertexAttributes.FindChecked(AttributeName);	
						ModelAttribute.Values.SetNumUninitialized(TargetLODModel.NumVertices);
						for(uint32 VertexIndex = 0; VertexIndex < TargetLODModel.NumVertices; VertexIndex++)
						{
							const int32 ImportVertexIndex = TargetLODModel.MeshToImportVertexMap[VertexIndex];
							ModelAttribute.Values[VertexIndex] = SourceAttribute.Get(FVertexID(ImportVertexIndex));
						}
					})
				);
			}
			else
			{
				// Initialize with zero data, matching the vertex count.
				ModelAttribute.Values.SetNumZeroed(TargetLODModel.NumVertices);
			}
		}
	}

	// Wait for all the attribute conversion tasks to complete.
	UE::Tasks::Wait(ConversionTasks);

	
	return true;
}


void FLODUtilities::RegenerateDependentLODs(USkeletalMesh* SkeletalMesh, int32 LODIndex, const ITargetPlatform* TargetPlatform)
{
	int32 LODNumber = SkeletalMesh->GetLODNum();
	TMap<int32, TArray<int32>> Dependencies;
	TBitArray<> DependentLOD;
	DependentLOD.Init(false, LODNumber);
	DependentLOD[LODIndex] = true;
	for (int32 DependentLODIndex = LODIndex + 1; DependentLODIndex < LODNumber; ++DependentLODIndex)
	{
		const FSkeletalMeshLODInfo* LODInfo = SkeletalMesh->GetLODInfo(DependentLODIndex);
		//Only add active reduction LOD that are not inline reducted (inline mean they do not depend on LODIndex)
		if (LODInfo && (SkeletalMesh->IsReductionActive(DependentLODIndex) || LODInfo->bHasBeenSimplified) && DependentLODIndex > LODInfo->ReductionSettings.BaseLOD && DependentLOD[LODInfo->ReductionSettings.BaseLOD])
		{
			TArray<int32>& LODDependencies = Dependencies.FindOrAdd(LODInfo->ReductionSettings.BaseLOD);
			LODDependencies.Add(DependentLODIndex);
			DependentLOD[DependentLODIndex] = true;
		}
	}
	if (Dependencies.Contains(LODIndex))
	{
		//Load the necessary module before going multithreaded
		IMeshReductionModule& ReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionModule>("MeshReductionInterface");
		//This will load all necessary module before kicking the multi threaded reduction
		IMeshReduction* MeshReduction = ReductionModule.GetSkeletalMeshReductionInterface();
		if (!MeshReduction)
		{
			UE_ASSET_LOG(LogLODUtilities, Warning, SkeletalMesh, TEXT("Cannot reduce skeletalmesh LOD because there is no active reduction plugin."));
			return;
		}
		check(MeshReduction->IsSupported());

		FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkeletalMesh);

		if (IsInGameThread())
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("DesiredLOD"), LODIndex);
			Args.Add(TEXT("SkeletalMeshName"), FText::FromString(SkeletalMesh->GetName()));
			const FText StatusUpdate = FText::Format(NSLOCTEXT("UnrealEd", "MeshSimp_GeneratingDependentLODs_F", "Generating All Dependent LODs from LOD {DesiredLOD} for {SkeletalMeshName}..."), Args);
			GWarn->BeginSlowTask(StatusUpdate, true);
		}

		for (const auto& Kvp : Dependencies)
		{
			int32 MaxDependentLODIndex = 0;
			//Use a TQueue which is thread safe, this Queue will be fill by some delegate call from other threads
			TQueue<FSkeletalMeshLODModel*> LODModelReplaceByReduction;

			const TArray<int32>& DependentLODs = Kvp.Value;
			//Clothing do not play well with multithread, backup it here. Also bind the LODModel delete delegates
			TMap<int32, TArray<ClothingAssetUtils::FClothingAssetMeshBinding>> PerLODClothingBindings;
			for (int32 DependentLODIndex : DependentLODs)
			{
				MaxDependentLODIndex = FMath::Max(MaxDependentLODIndex, DependentLODIndex);
				TArray<ClothingAssetUtils::FClothingAssetMeshBinding>& ClothingBindings = PerLODClothingBindings.FindOrAdd(DependentLODIndex);
				FLODUtilities::UnbindClothingAndBackup(SkeletalMesh, ClothingBindings, DependentLODIndex);

				const FSkeletalMeshLODInfo* LODInfo = SkeletalMesh->GetLODInfo(DependentLODIndex);
				check(LODInfo);
				
				LODInfo->ReductionSettings.OnDeleteLODModelDelegate.BindLambda([&LODModelReplaceByReduction](FSkeletalMeshLODModel* ReplacedLODModel)
				{
					LODModelReplaceByReduction.Enqueue(ReplacedLODModel);
				});
			}

			//Reduce all dependent LODs
			std::atomic<bool> bNeedsPackageDirtied{false};
			
			//Adjust the InlineReductionCacheDatas before simplifying dependent LODs
			if (SkeletalMesh->GetImportedModel()->InlineReductionCacheDatas.Num() < LODNumber)
			{
				SkeletalMesh->GetImportedModel()->InlineReductionCacheDatas.AddDefaulted(LODNumber - SkeletalMesh->GetImportedModel()->InlineReductionCacheDatas.Num());
			}
			else if (SkeletalMesh->GetImportedModel()->InlineReductionCacheDatas.Num() > LODNumber)
			{
				//If we have too much entry simply shrink the array to valid LODModel size
				SkeletalMesh->GetImportedModel()->InlineReductionCacheDatas.SetNum(LODNumber);
			}

			struct FSimplifyThreadContext
			{
				TArray<FMorphTargetUpdateRequest> MorphTargetUpdateRequests;
			};

			// Reduce LODs in parallel (reduction is multithread safe)
			const bool bHasAccessToLockedProperties = !FSkinnedAssetAsyncBuildScope::ShouldWaitOnLockedProperties(SkeletalMesh);
			TArray<FSimplifyThreadContext> SimplifyThreadContexts;
			ParallelForWithTaskContext(SimplifyThreadContexts, DependentLODs.Num(),
				[&DependentLODs, &SkeletalMesh, &bNeedsPackageDirtied, bHasAccessToLockedProperties, &TargetPlatform](FSimplifyThreadContext& Context, int32 IterationIndex)
			{
				TUniquePtr<FSkinnedAssetAsyncBuildScope> AsyncBuildScope(bHasAccessToLockedProperties ? MakeUnique<FSkinnedAssetAsyncBuildScope>(SkeletalMesh) : nullptr);

				check(DependentLODs.IsValidIndex(IterationIndex));
				int32 DependentLODIndex = DependentLODs[IterationIndex];
				check(SkeletalMesh->GetLODInfo(DependentLODIndex)); //We cannot add a LOD when reducing with multi thread, so check we already have one
				FLODUtilities::SimplifySkeletalMeshLOD(SkeletalMesh, DependentLODIndex, TargetPlatform, false, Context.MorphTargetUpdateRequests, &bNeedsPackageDirtied);
			});

			// Perform the morph target updates that we cannot do in a separate thread.
			for (FSimplifyThreadContext& Context: SimplifyThreadContexts)
			{
				for (FMorphTargetUpdateRequest& Request: Context.MorphTargetUpdateRequests)
				{
					Request.Run();
				}
			}

			if (bNeedsPackageDirtied && IsInGameThread())
			{
				SkeletalMesh->MarkPackageDirty();
			}

			//Restore the clothings and unbind the delegates
			for (int32 DependentLODIndex : DependentLODs)
			{
				TArray<ClothingAssetUtils::FClothingAssetMeshBinding>& ClothingBindings = PerLODClothingBindings.FindChecked(DependentLODIndex);
				FLODUtilities::RestoreClothingFromBackup(SkeletalMesh, ClothingBindings);

				FSkeletalMeshLODInfo* LODInfo = SkeletalMesh->GetLODInfo(DependentLODIndex);
				check(LODInfo);
				LODInfo->ReductionSettings.OnDeleteLODModelDelegate.Unbind();
			}

			while (!LODModelReplaceByReduction.IsEmpty())
			{
				FSkeletalMeshLODModel* ReplacedLODModel = nullptr;
				LODModelReplaceByReduction.Dequeue(ReplacedLODModel);
				if (ReplacedLODModel)
				{
					delete ReplacedLODModel;
				}
			}
			check(LODModelReplaceByReduction.IsEmpty());
		}

		if (IsInGameThread())
		{
			GWarn->EndSlowTask();
		}
	}
}


//////////////////////////////////////////////////////////////////////////
// Morph targets build code
//

struct FMeshDataBundle
{
	TArray< FVector3f > Vertices;
	TArray< uint32 > Indices;
	TArray< FVector2f > UVs;
	TArray< uint32 > SmoothingGroups;
	TArray<SkeletalMeshImportData::FTriangle> Faces;
};

static void ConvertImportDataToMeshData(const FSkeletalMeshImportData& ImportData, FMeshDataBundle& MeshDataBundle)
{
	for (const SkeletalMeshImportData::FTriangle& Face : ImportData.Faces)
	{
		SkeletalMeshImportData::FTriangle FaceTriangle;
		FaceTriangle = Face;
		for (int32 i = 0; i < 3; ++i)
		{
			const SkeletalMeshImportData::FVertex& Wedge = ImportData.Wedges[Face.WedgeIndex[i]];
			int32 FaceWedgeIndex = MeshDataBundle.Indices.Add(Wedge.VertexIndex);
			MeshDataBundle.UVs.Add(Wedge.UVs[0]);
			FaceTriangle.WedgeIndex[i] = FaceWedgeIndex;
		}
		MeshDataBundle.Faces.Add(FaceTriangle);
		MeshDataBundle.SmoothingGroups.Add(Face.SmoothingGroups);
	}

	MeshDataBundle.Vertices = ImportData.Points;
}

/**
* A class encapsulating morph target processing that occurs during import on a separate thread
*/
class FAsyncImportMorphTargetWork : public FNonAbandonableTask
{
public:
	FAsyncImportMorphTargetWork(
		const FName InMorphName,
		FSkeletalMeshLODModel* InLODModel, 
		const FReferenceSkeleton& InRefSkeleton,
		const FSkeletalMeshImportData& InBaseImportData,
		const FMeshDescription* InSkeletalMeshModel,
		TArray<FVector3f>&& InMorphLODPoints,
		TArray< FMorphTargetDelta >& InMorphDeltas,
		TArray<uint32>& InBaseIndexData,
		const TArray< uint32 >& InBaseWedgePointIndices,
		TMap<uint32, uint32>& InWedgePointToVertexIndexMap,
		const FOverlappingCorners& InOverlappingCorners,
		const TSet<uint32>& InModifiedPoints,
		const TMultiMap< int32, int32 >& InWedgeToFaces,
		const FMeshDataBundle& InMeshDataBundle,
		const TArray<FVector3f>& InTangentZ,
		bool InShouldImportNormals,
		bool InShouldImportTangents, 
		bool InbUseMikkTSpace, 
		const FOverlappingThresholds InThresholds
		)
		: LODModel(InLODModel)
		, RefSkeleton(InRefSkeleton)
		, BaseImportData(InBaseImportData)
		, SkeletalMeshModel(InSkeletalMeshModel)
		, CompressMorphLODPoints(InMorphLODPoints)
		, MorphTargetDeltas(InMorphDeltas)
		, BaseIndexData(InBaseIndexData)
		, BaseWedgePointIndices(InBaseWedgePointIndices)
		, WedgePointToVertexIndexMap(InWedgePointToVertexIndexMap)
		, OverlappingCorners(InOverlappingCorners)
		, ModifiedPoints(InModifiedPoints)
		, WedgeToFaces(InWedgeToFaces)
		, MeshDataBundle(InMeshDataBundle)
		, BaseTangentZ(InTangentZ)
		, TangentZ(InTangentZ)
		, ShouldImportNormals(InShouldImportNormals)
		, ShouldImportTangents(InShouldImportTangents)
		, bUseMikkTSpace(InbUseMikkTSpace)
		, Thresholds(InThresholds)
	{
		MeshUtilities = &FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");

		if (SkeletalMeshModel)
		{
			FSkeletalMeshConstAttributes SkeletalMeshAttributes(*SkeletalMeshModel);

			BaseNormalAttribute = SkeletalMeshAttributes.GetVertexInstanceNormals();
			MorphNormalDeltaAttribute = SkeletalMeshAttributes.GetVertexInstanceMorphNormalDelta(InMorphName);
			ShouldImportNormals = false;
		}
	}

	//Decompress the shape points data
	void DecompressData()
	{
		const TArray<FVector3f>& BaseMeshPoints = BaseImportData.Points;
		MorphLODPoints = BaseMeshPoints;
		int32 ModifiedPointIndex = 0;
		for (uint32 PointIndex : ModifiedPoints)
		{
			MorphLODPoints[PointIndex] = CompressMorphLODPoints[ModifiedPointIndex];
			ModifiedPointIndex++;
		}

		check(MorphLODPoints.Num() == MeshDataBundle.Vertices.Num());
	}

	void PrepareTangents()
	{
		TArray<bool> WasProcessed;
		WasProcessed.Empty(MeshDataBundle.Indices.Num());
		WasProcessed.AddZeroed(MeshDataBundle.Indices.Num());

		TArray< int32 > WedgeFaces;
		TArray< int32 > OtherWedgeFaces;
		TArray< int32 > OverlappingWedgesDummy;
		TArray< int32 > OtherOverlappingWedgesDummy;

		// For each ModifiedPoints, reset the tangents for the affected wedges
		for (int32 WedgeIdx = 0; WedgeIdx < MeshDataBundle.Indices.Num(); ++WedgeIdx)
		{
			int32 PointIdx = MeshDataBundle.Indices[WedgeIdx];

			if (ModifiedPoints.Find(PointIdx) != nullptr)
			{
				TangentZ[WedgeIdx] = FVector3f::ZeroVector;

				const TArray<int32>& OverlappingWedges = FindIncludingNoOverlapping(OverlappingCorners, WedgeIdx, OverlappingWedgesDummy);

				for (const int32 OverlappingWedgeIndex : OverlappingWedges)
				{
					if (WasProcessed[OverlappingWedgeIndex])
					{
						continue;
					}

					WasProcessed[OverlappingWedgeIndex] = true;

					WedgeFaces.Reset();
					WedgeToFaces.MultiFind(OverlappingWedgeIndex, WedgeFaces);

					for (const int32 FaceIndex : WedgeFaces)
					{
						for (int32 CornerIndex = 0; CornerIndex < 3; ++CornerIndex)
						{
							int32 WedgeIndex = MeshDataBundle.Faces[FaceIndex].WedgeIndex[CornerIndex];

							TangentZ[WedgeIndex] = FVector3f::ZeroVector;

							const TArray<int32>& OtherOverlappingWedges = FindIncludingNoOverlapping(OverlappingCorners, WedgeIndex, OtherOverlappingWedgesDummy);

							for (const int32 OtherDupVert : OtherOverlappingWedges)
							{
								OtherWedgeFaces.Reset();
								WedgeToFaces.MultiFind(OtherDupVert, OtherWedgeFaces);

								for (const int32 OtherFaceIndex : OtherWedgeFaces)
								{
									for (int32 OtherCornerIndex = 0; OtherCornerIndex < 3; ++OtherCornerIndex)
									{
										int32 OtherWedgeIndex = MeshDataBundle.Faces[OtherFaceIndex].WedgeIndex[OtherCornerIndex];

										TangentZ[OtherWedgeIndex] = FVector3f::ZeroVector;
									}
								}
							}
						}
					}
				}
			}
		}
	}

	void ComputeTangents()
	{
		bool bComputeNormals = !ShouldImportNormals || !BaseImportData.bHasNormals;
		bool bComputeTangents = !ShouldImportTangents || !BaseImportData.bHasTangents;
		bool bUseMikkTSpaceFinal = bUseMikkTSpace && (!ShouldImportNormals || !ShouldImportTangents);

		check(MorphLODPoints.Num() == MeshDataBundle.Vertices.Num());

		ETangentOptions::Type TangentOptions = ETangentOptions::BlendOverlappingNormals;

		// MikkTSpace should be use only when the user want to recompute the normals or tangents otherwise should always fallback on builtin
		if (bUseMikkTSpaceFinal && (bComputeNormals || bComputeTangents))
		{
			TangentOptions = (ETangentOptions::Type)(TangentOptions | ETangentOptions::UseMikkTSpace);
		}

		MeshUtilities->CalculateNormals(MorphLODPoints, MeshDataBundle.Indices, MeshDataBundle.UVs, MeshDataBundle.SmoothingGroups, TangentOptions, TangentZ);
	}

	void ComputeMorphDeltas()
	{
		TArray<bool> WasProcessed;
		WasProcessed.Empty(LODModel->NumVertices);
		WasProcessed.AddZeroed(LODModel->NumVertices);

		const bool bUseAttributeForNormal = MorphNormalDeltaAttribute.IsValid();

		for (int32 Idx = 0; Idx < BaseIndexData.Num(); ++Idx)
		{
			uint32 BaseVertIdx = BaseIndexData[Idx];
			// check for duplicate processing
			if (WasProcessed[BaseVertIdx])
			{
				continue;
			}
			
			// mark this base vertex as already processed
			WasProcessed[BaseVertIdx] = true;

			// clothing can add extra verts, and we won't have source point, so we ignore those
			if (!BaseWedgePointIndices.IsValidIndex(BaseVertIdx))
			{
				continue;
			}
			
			// get the base mesh's original wedge point index
			uint32 BasePointIdx = BaseWedgePointIndices[BaseVertIdx];
			if (MeshDataBundle.Vertices.IsValidIndex(BasePointIdx) && MorphLODPoints.IsValidIndex(BasePointIdx))
			{
				const FVector3f& BasePosition = MeshDataBundle.Vertices[BasePointIdx];
				const FVector3f& TargetPosition = MorphLODPoints[BasePointIdx];

				const FVector3f PositionDelta = TargetPosition - BasePosition;

				uint32* VertexIdx = WedgePointToVertexIndexMap.Find(BasePointIdx);

				FVector3f NormalDeltaZ = FVector3f::ZeroVector;

				if (VertexIdx != nullptr)
				{
					FVector3f BaseNormal = BaseTangentZ[*VertexIdx];

					if (bUseAttributeForNormal)
					{
						// BasePointIdx is the index into the import data points. These map directly to the mesh description's vertex ids.
						const FVertexID VertexID(BasePointIdx);
						const FVertexInstanceID VertexInstanceID = SkeletalMeshModel->GetVertexVertexInstanceIDs(VertexID)[0];
						// Use the first one. They all carry the same information for now due to the only place we get these normals are from
						// rebuilding from a FSkeletalMeshLODModel + UMorphTargets.
						NormalDeltaZ = MorphNormalDeltaAttribute.Get(VertexInstanceID);
					}
					else
					{
						NormalDeltaZ = TangentZ[*VertexIdx] - BaseNormal;
					}
				}

				// check if position actually changed much
				if (PositionDelta.SizeSquared() > FMath::Square(Thresholds.MorphThresholdPosition) ||
					// since we can't get imported morph target normal from FBX
					// we can't compare normal unless it's calculated
					// this is special flag to ignore normal diff
					((ShouldImportNormals == false) && NormalDeltaZ.SizeSquared() > 0.01f))
				{
					// create a new entry
					FMorphTargetDelta NewVertex;
					// position delta
					NewVertex.PositionDelta = PositionDelta;
					// normal delta
					NewVertex.TangentZDelta = NormalDeltaZ;
					// index of base mesh vert this entry is to modify
					NewVertex.SourceIdx = BaseVertIdx;

					// add it to the list of changed verts
					MorphTargetDeltas.Add(NewVertex);
				}
			}
		}
	}

	void DoWork()
	{
		DecompressData();
		if (!MorphNormalDeltaAttribute.IsValid())
		{
			PrepareTangents();
			ComputeTangents();
		}
		ComputeMorphDeltas();
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncImportMorphTargetWork, STATGROUP_ThreadPoolAsyncTasks);
	}

private:

	const TArray<int32>& FindIncludingNoOverlapping(const FOverlappingCorners& Corners, int32 Key, TArray<int32>& NoOverlapping)
	{
		const TArray<int32>& Found = Corners.FindIfOverlapping(Key);
		if (Found.Num() > 0)
		{
			return Found;
		}
		else
		{
			NoOverlapping.Reset(1);
			NoOverlapping.Add(Key);
			return NoOverlapping;
		}
	}

	FSkeletalMeshLODModel* LODModel;
	// @todo not thread safe
	const FReferenceSkeleton& RefSkeleton;
	const FSkeletalMeshImportData& BaseImportData;
	const FMeshDescription* SkeletalMeshModel;
	TVertexInstanceAttributesConstRef<FVector3f> BaseNormalAttribute;
	TVertexInstanceAttributesConstRef<FVector3f> MorphNormalDeltaAttribute;
	const TArray<FVector3f> CompressMorphLODPoints;
	TArray<FVector3f> MorphLODPoints;

	IMeshUtilities* MeshUtilities;

	TArray< FMorphTargetDelta >& MorphTargetDeltas;
	TArray< uint32 >& BaseIndexData;
	const TArray< uint32 >& BaseWedgePointIndices;
	TMap<uint32, uint32>& WedgePointToVertexIndexMap;

	const FOverlappingCorners& OverlappingCorners;
	const TSet<uint32>& ModifiedPoints;
	const TMultiMap< int32, int32 >& WedgeToFaces;
	const FMeshDataBundle& MeshDataBundle;

	const TArray<FVector3f>& BaseTangentZ;
	TArray<FVector3f> TangentZ;
	bool ShouldImportNormals;
	bool ShouldImportTangents;
	bool bUseMikkTSpace;
	const FOverlappingThresholds Thresholds;
};


static void BuildMorphTargetsInternal(
	USkeletalMesh* BaseSkelMesh,
	const FMeshDescription* SkeletalMeshModel,
	FSkeletalMeshImportData &BaseImportData,
	int32 LODIndex,
	bool ShouldImportNormals,
	bool ShouldImportTangents,
	bool bUseMikkTSpace,
	const FOverlappingThresholds& Thresholds
	)
{
	bool bComputeNormals = !ShouldImportNormals || !BaseImportData.bHasNormals;
	bool bComputeTangents = !ShouldImportTangents || !BaseImportData.bHasTangents;
	bool bUseMikkTSpaceFinal = bUseMikkTSpace && (!ShouldImportNormals || !ShouldImportTangents);

	// Prepare base data
	FSkeletalMeshLODModel& BaseLODModel = BaseSkelMesh->GetImportedModel()->LODModels[LODIndex];
	const FSkeletalMeshLODInfo* BaseLodInfo = BaseSkelMesh->GetLODInfo(LODIndex);

	FMeshDataBundle MeshDataBundle;
	ConvertImportDataToMeshData(BaseImportData, MeshDataBundle);

	IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");

	ETangentOptions::Type TangentOptions = ETangentOptions::BlendOverlappingNormals;

	// MikkTSpace should be use only when the user want to recompute the normals or tangents otherwise should always fallback on builtin
	if (bUseMikkTSpaceFinal && (bComputeNormals || bComputeTangents))
	{
		TangentOptions = (ETangentOptions::Type)(TangentOptions | ETangentOptions::UseMikkTSpace);
	}

	FOverlappingCorners OverlappingVertices;
	MeshUtilities.CalculateOverlappingCorners(MeshDataBundle.Vertices, MeshDataBundle.Indices, false, OverlappingVertices);

	TArray<FVector3f> TangentZ;
	MeshUtilities.CalculateNormals(MeshDataBundle.Vertices, MeshDataBundle.Indices, MeshDataBundle.UVs, MeshDataBundle.SmoothingGroups, TangentOptions, TangentZ);

	TArray<uint32> BaseIndexData = BaseLODModel.IndexBuffer;

	TMap<uint32, uint32> WedgePointToVertexIndexMap;
	// Build a mapping of wedge point indices to vertex indices for fast lookup later.
	for (int32 Idx = 0; Idx < MeshDataBundle.Indices.Num(); ++Idx)
	{
		WedgePointToVertexIndexMap.Add(MeshDataBundle.Indices[Idx], Idx);
	}

	// Create a map from wedge indices to faces
	TMultiMap< int32, int32 > WedgeToFaces;
	for (int32 FaceIndex = 0; FaceIndex < MeshDataBundle.Faces.Num(); FaceIndex++)
	{
		const SkeletalMeshImportData::FTriangle& Face = MeshDataBundle.Faces[FaceIndex];
		for (int32 CornerIndex = 0; CornerIndex < 3; ++CornerIndex)
		{
			WedgeToFaces.AddUnique(Face.WedgeIndex[CornerIndex], FaceIndex);
		}
	}

	// Temp arrays to keep track of data being used by threads
	TArray< TArray< FMorphTargetDelta >* > Results;
	TArray<UMorphTarget*> MorphTargets;

	// Array of pending tasks that are not complete
	TIndirectArray<FAsyncTask<FAsyncImportMorphTargetWork> > PendingWork;

	int32 NumCompleted = 0;
	int32 NumTasks = 0;
	int32 MaxShapeInProcess = FPlatformMisc::NumberOfCoresIncludingHyperthreads();

	int32 ShapeIndex = 0;
	int32 TotalShapeCount = BaseImportData.MorphTargetNames.Num();

	TMap<FName, UMorphTarget*> ExistingMorphTargets;
	for (UMorphTarget* MorphTarget : BaseSkelMesh->GetMorphTargets())
	{
		ExistingMorphTargets.Add(MorphTarget->GetFName(), MorphTarget);
	}

	// iterate through shapename, and create morphtarget
	for (int32 MorphTargetIndex = 0; MorphTargetIndex < BaseImportData.MorphTargetNames.Num(); ++MorphTargetIndex)
	{
		int32 CurrentNumTasks = PendingWork.Num();
		while (CurrentNumTasks >= MaxShapeInProcess)
		{
			//Wait until the first slot is available
			PendingWork[0].EnsureCompletion();
			for (int32 TaskIndex = PendingWork.Num() - 1; TaskIndex >= 0; --TaskIndex)
			{
				if (PendingWork[TaskIndex].IsDone())
				{
					PendingWork.RemoveAt(TaskIndex);
					++NumCompleted;
					if (IsInGameThread())
					{
						FFormatNamedArguments Args;
						Args.Add(TEXT("NumCompleted"), NumCompleted);
						Args.Add(TEXT("NumTasks"), TotalShapeCount);
						GWarn->StatusUpdate(NumCompleted, TotalShapeCount, FText::Format(LOCTEXT("ImportingMorphTargetStatus", "Importing Morph Target: {NumCompleted} of {NumTasks}"), Args));
					}
				}
			}
			CurrentNumTasks = PendingWork.Num();
		}

		check(BaseImportData.MorphTargetNames.IsValidIndex(MorphTargetIndex));
		check(BaseImportData.MorphTargetModifiedPoints.IsValidIndex(MorphTargetIndex));
		check(BaseImportData.MorphTargets.IsValidIndex(MorphTargetIndex));

		FString& ShapeName = BaseImportData.MorphTargetNames[MorphTargetIndex];
		FSkeletalMeshImportData& ShapeImportData = BaseImportData.MorphTargets[MorphTargetIndex];
		TSet<uint32>& ModifiedPoints = BaseImportData.MorphTargetModifiedPoints[MorphTargetIndex];

		UMorphTarget* MorphTarget = nullptr;
		{
			FName ObjectName = *ShapeName;
			MorphTarget = ExistingMorphTargets.FindRef(ObjectName);

			// we only create new one for LOD0, otherwise don't create new one
			if (!MorphTarget)
			{
				if (LODIndex == 0)
				{
					FGCScopeGuard Guard;

					// Check if there's an orphaned morph target object floating about. We use StaticFindObjectFastInternal
					// rather than StaticFindObjectFast since we might have a situation where the orphaned morph target might
					// still have the Async flag set, which StaticFindObjectFast will skip over.
					MorphTarget = Cast<UMorphTarget>(StaticFindObjectFastInternal(UMorphTarget::StaticClass(), BaseSkelMesh, ObjectName, EFindObjectFlags::ExactClass));
					if (MorphTarget)
					{
						// Temporarily set the async flag on it so that it doesn't vanish once we leave the GC scope.
						MorphTarget->SetInternalFlags(EInternalObjectFlags::Async);
						
						// Eviscerate this morph target for 
						MorphTarget->GetMorphLODModels().Reset();
					}
					else
					{
						// This will already have the async flag on it, since we're creating the object in a worker thread.
						MorphTarget = NewObject<UMorphTarget>(BaseSkelMesh, ObjectName);
					}
				}
			}
		}

		if (MorphTarget)
		{
			check(IsValid(MorphTarget));
			MorphTargets.Add(MorphTarget);
			int32 NewMorphDeltasIdx = Results.Add(new TArray< FMorphTargetDelta >());

			TArray< FMorphTargetDelta >* Deltas = Results[NewMorphDeltasIdx];

			FAsyncTask<FAsyncImportMorphTargetWork>* NewWork = new FAsyncTask<FAsyncImportMorphTargetWork>(MorphTarget->GetFName(), &BaseLODModel, BaseSkelMesh->GetRefSkeleton(), BaseImportData, SkeletalMeshModel,
				MoveTemp(ShapeImportData.Points), *Deltas, BaseIndexData, BaseLODModel.GetRawPointIndices(), WedgePointToVertexIndexMap, OverlappingVertices, ModifiedPoints, WedgeToFaces, MeshDataBundle, TangentZ,
				ShouldImportNormals, ShouldImportTangents, bUseMikkTSpace, Thresholds);
			PendingWork.Add(NewWork);

			NewWork->StartBackgroundTask(GLargeThreadPool);
			CurrentNumTasks++;
			NumTasks++;
		}

		++ShapeIndex;
	}

	// Wait for all importing tasks to complete
	for (int32 TaskIndex = 0; TaskIndex < PendingWork.Num(); ++TaskIndex)
	{
		PendingWork[TaskIndex].EnsureCompletion();

		++NumCompleted;

		if (IsInGameThread())
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("NumCompleted"), NumCompleted);
			Args.Add(TEXT("NumTasks"), TotalShapeCount);
			GWarn->StatusUpdate(NumCompleted, NumTasks, FText::Format(LOCTEXT("ImportingMorphTargetStatus", "Importing Morph Target: {NumCompleted} of {NumTasks}"), Args));
		}
	}

	bool bNeedToInvalidateRegisteredMorph = false;
	// Create morph streams for each morph target we are importing.
	// This has to happen on a single thread since the skeletal meshes' bulk data is locked and cant be accessed by multiple threads simultaneously
	for (int32 Index = 0; Index < MorphTargets.Num(); Index++)
	{
		if (IsInGameThread())
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("NumCompleted"), Index + 1);
			Args.Add(TEXT("NumTasks"), MorphTargets.Num());
			GWarn->StatusUpdate(Index + 1, MorphTargets.Num(), FText::Format(LOCTEXT("BuildingMorphTargetRenderDataStatus", "Building Morph Target Render Data: {NumCompleted} of {NumTasks}"), Args));
		}

		UMorphTarget* MorphTarget = MorphTargets[Index];
		check(IsValid(MorphTarget));
		constexpr bool bCompareNormals = true;		// Ensure we include morphs that only modify normals and not positions. 
		constexpr bool bGeneratedByReduction = false;
		MorphTarget->PopulateDeltas(*Results[Index], LODIndex, BaseLODModel.Sections, bCompareNormals, bGeneratedByReduction, Thresholds.MorphThresholdPosition);

		// register does mark package as dirty
		if (MorphTarget->HasValidData())
		{
			bNeedToInvalidateRegisteredMorph |= BaseSkelMesh->RegisterMorphTarget(MorphTarget, false);
		}

		delete Results[Index];
		Results[Index] = nullptr;

		//Reset the morph target imported source filename
		MorphTarget->SetCustomImportedSourceFilename(LODIndex, FString());
		MorphTarget->SetGeneratedByEngine(LODIndex, false);
		//Now set the imported source filename from the imported data
		if (BaseLodInfo)
		{
			if (const FMorphTargetImportedSourceFileInfo* MorphTargetImportedSourceFileInfo = BaseLodInfo->ImportedMorphTargetSourceFilename.Find(MorphTarget->GetName()))
			{
				const FString& SourceFilename = MorphTargetImportedSourceFileInfo->GetSourceFilename();
				if (MorphTargetImportedSourceFileInfo->IsGeneratedByEngine())
				{
					MorphTarget->SetGeneratedByEngine(LODIndex, true);
				}
				else if (!SourceFilename.IsEmpty())
				{
					MorphTarget->SetCustomImportedSourceFilename(LODIndex, *SourceFilename);
				}
			}
		}

		// We might have created new MorphTarget in an async thread, so we need to remove the async flag so they can get
		// garbage collected in the future now that their references are properly setup and reachable by the GC.
		MorphTarget->ClearInternalFlags(EInternalObjectFlags::Async);
	}

	if (bNeedToInvalidateRegisteredMorph)
	{
		BaseSkelMesh->InitMorphTargetsAndRebuildRenderData();
	}
}

void FLODUtilities::BuildMorphTargets(USkeletalMesh* SkeletalMesh, FSkeletalMeshImportData& ImportData, int32 LODIndex, bool ShouldImportNormals, bool ShouldImportTangents, bool bUseMikkTSpace, const FOverlappingThresholds& Thresholds)
{
	BuildMorphTargetsInternal(SkeletalMesh, nullptr, ImportData, LODIndex, ShouldImportNormals, ShouldImportTangents, bUseMikkTSpace, Thresholds);
}

void FLODUtilities::BuildMorphTargets(USkeletalMesh* SkeletalMesh, const FMeshDescription& SkeletalMeshModel, FSkeletalMeshImportData& ImportData, int32 LODIndex, bool ShouldImportNormals, bool ShouldImportTangents, bool bUseMikkTSpace, const FOverlappingThresholds& Thresholds)
{
	BuildMorphTargetsInternal(SkeletalMesh, &SkeletalMeshModel, ImportData, LODIndex, ShouldImportNormals, ShouldImportTangents, bUseMikkTSpace, Thresholds);
}



void FLODUtilities::UnbindClothingAndBackup(USkeletalMesh* SkeletalMesh, TArray<ClothingAssetUtils::FClothingAssetMeshBinding>& ClothingBindings)
{
	ClothingBindings.Reset();

	if (ClothingAssetUtils::HasAnyMeshClothingAssetBindings(SkeletalMesh))  // Don't run the PostEditChange unless strictly necessary
	{
		FScopedSkeletalMeshPostEditChange SkeletalMeshPostEditChange(SkeletalMesh);

		for (int32 LODIndex = 0; LODIndex < SkeletalMesh->GetImportedModel()->LODModels.Num(); ++LODIndex)
		{
			TArray<ClothingAssetUtils::FClothingAssetMeshBinding> LODBindings;
			UnbindClothingAndBackup(SkeletalMesh, LODBindings, LODIndex);
			ClothingBindings.Append(LODBindings);
		}
	}
}

void FLODUtilities::UnbindClothingAndBackup(USkeletalMesh* SkeletalMesh, TArray<ClothingAssetUtils::FClothingAssetMeshBinding>& ClothingBindings, const int32 LODIndex)
{
	if (!ClothingAssetUtils::HasAnyLodMeshClothingAssetBindings(SkeletalMesh, LODIndex))  // Early exit to avoid the FScopedSkeletalMeshPostEditChange whenever possible
	{
		return;
	}
	// Guard the unbinding with a scoped PostEditChange
	FScopedSkeletalMeshPostEditChange SkeletalMeshPostEditChange(SkeletalMesh);
	// Store the clothBinding
	ClothingAssetUtils::GetAllLodMeshClothingAssetBindings(SkeletalMesh, ClothingBindings, LODIndex);
	// Unbind the Cloth for this LOD before we reduce it, we will put back the cloth after the reduction, if it still match the sections
	FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[LODIndex];
	for (ClothingAssetUtils::FClothingAssetMeshBinding& Binding : ClothingBindings)
	{
		if (Binding.LODIndex == LODIndex)
		{
			// Use the UserSectionsData original section index, this will ensure we remap correctly the cloth if the reduction has change the number of sections
			int32 OriginalDataSectionIndex = LODModel.Sections[Binding.SectionIndex].OriginalDataSectionIndex;
			if (Binding.Asset)
			{
				Binding.Asset->UnbindFromSkeletalMesh(SkeletalMesh, Binding.LODIndex, INDEX_NONE);
				Binding.SectionIndex = OriginalDataSectionIndex;
			}
			
			if (FSkelMeshSourceSectionUserData* const SectionUserData = LODModel.UserSectionsData.Find(OriginalDataSectionIndex);
				ensureMsgf(SectionUserData, TEXT("A FScopedSkeletalMeshPostEditChange guard must be used to ensure that the model isn't getting rebuilt while this function is called.")))
			{
				SectionUserData->ClothingData.AssetGuid = FGuid();
				SectionUserData->ClothingData.AssetLodIndex = INDEX_NONE;
				SectionUserData->CorrespondClothAssetIndex = INDEX_NONE;
			}
		}
	}
}

void FLODUtilities::RestoreClothingFromBackup(USkeletalMesh* SkeletalMesh, TArray<ClothingAssetUtils::FClothingAssetMeshBinding>& ClothingBindings)
{
	if (!SkeletalMesh ||
		!SkeletalMesh->GetImportedModel() ||
		!ClothingBindings.Num())  // Early exit to avoid the FScopedSkeletalMeshPostEditChange whenever possible
	{
		return;
	}
	// Guard the binding with a scoped PostEditChange
	FScopedSkeletalMeshPostEditChange SkeletalMeshPostEditChange(SkeletalMesh);
	for (int32 LODIndex = 0; LODIndex < SkeletalMesh->GetImportedModel()->LODModels.Num(); ++LODIndex)
	{
		RestoreClothingFromBackup(SkeletalMesh, ClothingBindings, LODIndex);
	}
}

void FLODUtilities::RestoreClothingFromBackup(USkeletalMesh* SkeletalMesh, TArray<ClothingAssetUtils::FClothingAssetMeshBinding>& ClothingBindings, const int32 LODIndex)
{
	if (!SkeletalMesh ||
		!SkeletalMesh->GetImportedModel() ||
		!SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex) ||
		!ClothingBindings.Num())  // Early exit to avoid the FScopedSkeletalMeshPostEditChange whenever possible
	{
		return;
	}
	// Guard the binding with a scoped PostEditChange
	FScopedSkeletalMeshPostEditChange SkeletalMeshPostEditChange(SkeletalMesh);

	FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[LODIndex];
	for (ClothingAssetUtils::FClothingAssetMeshBinding& Binding : ClothingBindings)
	{
		for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); ++SectionIndex)
		{
			if (LODModel.Sections[SectionIndex].OriginalDataSectionIndex != Binding.SectionIndex)
			{
				continue;
			}
			if (Binding.LODIndex == LODIndex && Binding.Asset)
			{
				if (Binding.Asset->BindToSkeletalMesh(SkeletalMesh, Binding.LODIndex, SectionIndex, Binding.AssetInternalLodIndex))
				{
					// If successfull set back the section user data
					if (FSkelMeshSourceSectionUserData* const SectionUserData = LODModel.UserSectionsData.Find(Binding.SectionIndex);
						ensureMsgf(SectionUserData, TEXT("A FScopedSkeletalMeshPostEditChange guard must be used to ensure that the model isn't getting rebuilt while this function is called.")))
					{
						SectionUserData->CorrespondClothAssetIndex = LODModel.Sections[SectionIndex].CorrespondClothAssetIndex;
						SectionUserData->ClothingData = LODModel.Sections[SectionIndex].ClothingData;
					}
				}
			}
			break;
		}
	}
}

void FLODUtilities::BackupCustomImportedMorphTargetData(USkeletalMesh* SkeletalMesh, TMap<FString, TArray<FMorphTargetLodBackupData>>& BackupImportedMorphTargetData)
{
	const int32 LodCount = SkeletalMesh->GetLODNum();
	constexpr int32 LodIndex0 = 0;
	//Find all imported morph targets
	if (const FSkeletalMeshLODInfo* LodInfo = SkeletalMesh->GetLODInfo(LodIndex0))
	{
		TSet<FString> ImportedMorphTargetNames;
		LodInfo->ImportedMorphTargetSourceFilename.GetKeys(ImportedMorphTargetNames);
		for (const FString& MorphTargetNameStr : ImportedMorphTargetNames)
		{
			FName MorphTargetName(*MorphTargetNameStr);
			TArray<FMorphTargetLodBackupData>& MorphTargetLodDatas = BackupImportedMorphTargetData.FindOrAdd(MorphTargetNameStr);
			for (int32 LodIndex = 0; LodIndex < LodCount; ++LodIndex)
			{
				FMorphTargetLodBackupData& MorphTargetLodData = MorphTargetLodDatas.AddDefaulted_GetRef();
				if (SkeletalMesh->HasMeshDescription(LodIndex))
				{
					FMeshDescription& MeshDescription = *(SkeletalMesh->GetMeshDescription(LodIndex));
					FSkeletalMeshConstAttributes SkeletalMeshAttributes(MeshDescription);

					if (SkeletalMeshAttributes.HasMorphTargetPositionsAttribute(MorphTargetName))
					{
						MorphTargetLodData.bIsEmpty = false;
						MorphTargetLodData.MorphPositionDeltas.Reserve(MeshDescription.Vertices().Num());
						TVertexAttributesConstRef<FVector3f> VertexPositionDeltas = SkeletalMeshAttributes.GetVertexMorphPositionDelta(MorphTargetName);
						for (FVertexID VertexID : MeshDescription.Vertices().GetElementIDs())
						{
							MorphTargetLodData.MorphPositionDeltas.Add(VertexPositionDeltas[VertexID]);
						}
						if (SkeletalMeshAttributes.HasMorphTargetNormalsAttribute(MorphTargetName))
						{
							MorphTargetLodData.MorphNormalDeltas.Reserve(MeshDescription.VertexInstances().Num());
							TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceNormalDeltas = SkeletalMeshAttributes.GetVertexInstanceMorphNormalDelta(MorphTargetName);
							for (FVertexInstanceID VertexInstanceID : MeshDescription.VertexInstances().GetElementIDs())
							{
								MorphTargetLodData.MorphNormalDeltas.Add(VertexInstanceNormalDeltas[VertexInstanceID]);
							}
						}
					}
				}
			}
		}
	}
}

bool FLODUtilities::RestoreCustomImportedMorphTargetData(USkeletalMesh* SkeletalMesh, const int32 LodIndex, FMeshDescription& LodMeshDescription, const TMap<FString, TArray<FMorphTargetLodBackupData>>& BackupImportedMorphTargetData)
{
	TMap<FString, FMorphTargetImportedSourceFileInfo> RemovedImportedMorphTargetInfo;
	auto RemoveInvalidMorphTargetLOD = [SkeletalMesh, LodIndex, &RemovedImportedMorphTargetInfo](const FString& MorphTargetNameStr)
		{
			FName MorphTargetName(*MorphTargetNameStr);
			if(UMorphTarget* MorphTarget = SkeletalMesh->FindMorphTarget(MorphTargetName))
			{
				if (MorphTarget)
				{
					MorphTarget->RemoveFromRoot();
					MorphTarget->ClearFlags(RF_Standalone);

					if (SkeletalMesh->HasMeshDescription(LodIndex))
					{
						//Remove the morph target from the raw import data
						FMeshDescription* MeshDescription = SkeletalMesh->GetMeshDescription(LodIndex);
						FSkeletalMeshAttributes MeshAttributes(*MeshDescription);

						if (MeshAttributes.GetMorphTargetNames().Contains(MorphTargetName))
						{
							SkeletalMesh->ModifyMeshDescription(LodIndex);
							MeshAttributes.UnregisterMorphTargetAttribute(MorphTargetName);
							SkeletalMesh->CommitMeshDescription(LodIndex);
						}
					}

					SkeletalMesh->UnregisterMorphTarget(MorphTarget, false);
				}
			}
			if (FSkeletalMeshLODInfo* LodInfo = SkeletalMesh->GetLODInfo(LodIndex))
			{
				if (LodInfo->ImportedMorphTargetSourceFilename.Contains(MorphTargetNameStr))
				{
					FMorphTargetImportedSourceFileInfo& RemoveBackup = RemovedImportedMorphTargetInfo.FindOrAdd(MorphTargetNameStr);
					RemoveBackup = LodInfo->ImportedMorphTargetSourceFilename.FindChecked(MorphTargetNameStr);
					LodInfo->ImportedMorphTargetSourceFilename.Remove(MorphTargetNameStr);
				}
			}
		};
	
	bool bIsMeshDescriptionModified = false;
	for (const TPair<FString, TArray<FMorphTargetLodBackupData>>& ImportedMorphTargetPair : BackupImportedMorphTargetData)
	{
		const FString& MorphTargetNameStr = ImportedMorphTargetPair.Key;
		FName MorphTargetName(*MorphTargetNameStr);

		const TArray<FMorphTargetLodBackupData>& MorphTargetLodDatas = ImportedMorphTargetPair.Value;
		if (!MorphTargetLodDatas.IsValidIndex(LodIndex))
		{
			continue;
		}

		const FMorphTargetLodBackupData& MorphTargetLodData = MorphTargetLodDatas[LodIndex];
		if (MorphTargetLodData.bIsEmpty)
		{
			//In case the Source Skeletal didn't have MorphTarget for a given LodIndex, it would show up as an Empty.
			continue;
		}

		FSkeletalMeshAttributes SkeletalMeshAttributes(LodMeshDescription);
		if (SkeletalMeshAttributes.IsMorphTargetAttribute(MorphTargetName))
		{
			//The morph target already exist, this mean the re-import added this morph target
			continue;
		}

		if (MorphTargetLodData.MorphPositionDeltas.Num() != LodMeshDescription.Vertices().Num())
		{
			RemoveInvalidMorphTargetLOD(MorphTargetNameStr);
			//Error out that the morph target is not valid anymore for this LOD
			UE_ASSET_LOG(LogLODUtilities, Error, SkeletalMesh, TEXT("Cannot keep custom morph target %s for LOD %d, because the re-import LOD now have a different mesh geometry topology."), *MorphTargetNameStr, LodIndex);
			continue;
		}

		//We are good to go and we can add the data to the mesh description
		bIsMeshDescriptionModified = true;
		const bool bHasNormal = MorphTargetLodData.MorphNormalDeltas.Num() > 0;
		SkeletalMeshAttributes.RegisterMorphTargetAttribute(MorphTargetName, bHasNormal);
		TVertexAttributesRef<FVector3f> VertexPositionDeltas = SkeletalMeshAttributes.GetVertexMorphPositionDelta(MorphTargetName);
		for (int32 VertexIndex = 0; VertexIndex < MorphTargetLodData.MorphPositionDeltas.Num(); ++VertexIndex)
		{
			FVertexID VertexID(VertexIndex);
			VertexPositionDeltas[VertexID] = MorphTargetLodData.MorphPositionDeltas[VertexIndex];
		}
		if (bHasNormal)
		{
			TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormalDeltas = SkeletalMeshAttributes.GetVertexInstanceMorphNormalDelta(MorphTargetName);
			for (int32 VertexInstanceIndex = 0; VertexInstanceIndex < MorphTargetLodData.MorphPositionDeltas.Num(); ++VertexInstanceIndex)
			{
				FVertexInstanceID VertexInstanceID(VertexInstanceIndex);
				VertexInstanceNormalDeltas[VertexInstanceID] = MorphTargetLodData.MorphNormalDeltas[VertexInstanceIndex];
			}
		}
	}
	if (!GIsRunningUnattendedScript && !FApp::IsUnattended() && !RemovedImportedMorphTargetInfo.IsEmpty())
	{
		FTextBuilder ErrorMessage;
		ErrorMessage.AppendLine(LOCTEXT("BadTopologyMorphRemoveMessage", "Morph target(s) removed because of topology changes:"));
		ErrorMessage.Indent();
		for (const TPair<FString, FMorphTargetImportedSourceFileInfo>& RemovedMorphTargetInfo : RemovedImportedMorphTargetInfo)
		{
			if (RemovedMorphTargetInfo.Value.IsGeneratedByEngine())
			{
				ErrorMessage.AppendLine(FText::Format(LOCTEXT("BadTopologyMorphRemoveMessagePerMorphGenerated", "{0} [LOD {1}] Generate by Engine tools"), FText::FromString(RemovedMorphTargetInfo.Key), FText::AsNumber(LodIndex)));
			}
			else
			{
				ErrorMessage.AppendLine(FText::Format(LOCTEXT("BadTopologyMorphRemoveMessagePerMorphCustomImport", "{0} [LOD {1}] SourceFile:{2}"), FText::FromString(RemovedMorphTargetInfo.Key), FText::AsNumber(LodIndex), FText::FromString(RemovedMorphTargetInfo.Value.GetSourceFilename())));
			}
		}
		ErrorMessage.Unindent();
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage.ToText());
	}

	return bIsMeshDescriptionModified;
}

void FLODUtilities::AdjustImportDataFaceMaterialIndex(const TArray<FSkeletalMaterial>& Materials, TArray<SkeletalMeshImportData::FMaterial>& RawMeshMaterials, TArray<SkeletalMeshImportData::FMeshFace>& LODFaces, int32 LODIndex)
{
	if ((Materials.Num() <= 1 && RawMeshMaterials.Num() <= 1) || LODIndex != 0)
	{
		//Nothing to fix if we have 1 or less material or we are not adjusting the base LOD
		return;
	}

	//Fix the material for the faces
	TArray<int32> MaterialRemap;
	MaterialRemap.Reserve(RawMeshMaterials.Num());
	//Optimization to avoid doing the remap if no material have to change
	bool bNeedRemapping = false;
	bool bFoundMaterial = false;
	for (int32 MaterialIndex = 0; MaterialIndex < RawMeshMaterials.Num(); ++MaterialIndex)
	{
		MaterialRemap.Add(MaterialIndex);
		FName MaterialImportName = *(RawMeshMaterials[MaterialIndex].MaterialImportName);
		for (int32 MeshMaterialIndex = 0; MeshMaterialIndex < Materials.Num(); ++MeshMaterialIndex)
		{
			FName MeshMaterialName = Materials[MeshMaterialIndex].ImportedMaterialSlotName;
			if (MaterialImportName == MeshMaterialName)
			{
				bNeedRemapping |= (MaterialRemap[MaterialIndex] != MeshMaterialIndex);
				MaterialRemap[MaterialIndex] = MeshMaterialIndex;
				bFoundMaterial = true;
				break;
			}
		}
		//A material slot was deleted by the user, we cannot remap anything in that case, the LODMaterialMap should be active for the LOD 0 and will match the sections
		if (!bFoundMaterial)
		{
			return;
		}
	}
	if (bNeedRemapping)
	{
		//Make sure the data is good before doing the change, We cannot do the remap if we
		//have a bad synchronization between the face data and the Materials data.
		for (int32 FaceIndex = 0; FaceIndex < LODFaces.Num(); ++FaceIndex)
		{
			if (!MaterialRemap.IsValidIndex(LODFaces[FaceIndex].MeshMaterialIndex))
			{
				return;
			}
		}

		//Update all the faces
		for (int32 FaceIndex = 0; FaceIndex < LODFaces.Num(); ++FaceIndex)
		{
			LODFaces[FaceIndex].MeshMaterialIndex = static_cast<uint16>(MaterialRemap[LODFaces[FaceIndex].MeshMaterialIndex]);
		}
	}
}

void FLODUtilities::MatchImportedMaterials(FLODUtilities::FSkeletalMeshMatchImportedMaterialsParameters& Parameters)
{
	//Make sure we have valid parameters
	if (!ensure(Parameters.SkeletalMesh))
	{
		UE_LOGF(LogLODUtilities, Warning, "FLODUtilities::MatchImportedMaterials: Bad parameters, SkeletalMesh is null");
		return;
	}
	if (!ensure(Parameters.ImportedMaterials))
	{
		UE_ASSET_LOG(LogLODUtilities, Warning, Parameters.SkeletalMesh, TEXT("FLODUtilities::MatchImportedMaterials: Bad parameters, ImportedMaterials is null"));
		return;
	}
	if (Parameters.bIsReImport)
	{
		if (!ensure(Parameters.ExistingOriginalPerSectionMaterialImportName))
		{
			UE_ASSET_LOG(LogLODUtilities, Warning, Parameters.SkeletalMesh, TEXT("FLODUtilities::MatchImportedMaterials: Bad parameters, ExistingOriginalPerSectionMaterialImportName is null"));
			return;
		}

		if (Parameters.SkeletalMesh->GetMaterials().Num() == 0)
		{
			return;
		}
	}
	FSkeletalMeshLODInfo* LodInfo = Parameters.SkeletalMesh->GetLODInfo(Parameters.LodIndex);
	if (!LodInfo)
	{
		UE_ASSET_LOG(LogLODUtilities, Warning, Parameters.SkeletalMesh, TEXT("FLODUtilities::MatchImportedMaterials: LodIndex %d, LodInfo is null"), Parameters.LodIndex);
		return;
	}
	//Grab some parameters pointer into references
	const TArray<SkeletalMeshImportData::FMaterial>& ImportedMaterials = *Parameters.ImportedMaterials;

	TArray<FSkeletalMaterial>& Materials = Parameters.SkeletalMesh->GetMaterials();

	TMap<FName, int32> LODMaterialMapRedirection;
	TMap<FName, FSkelMeshSourceSectionUserData> NewUserSectionsDataMap;
	//If we reimport we have to keep the existing user section info data. We use the material name to match the existing
	if (Parameters.bIsReImport)
	{
		const TArray<FName>& ExistingOriginalPerSectionMaterialImportName = *Parameters.ExistingOriginalPerSectionMaterialImportName;

		auto GetImportMaterialSlotName = [&Parameters, &LodInfo](const FSkelMeshSection& Section
			, int32 SectionIndex
			, int32& OutMaterialIndex
			, const TArray<FName>& PerSectionMaterialImportName)->FName
			{
				const TArray<FSkeletalMaterial>& MeshMaterials = Parameters.SkeletalMesh->GetMaterials();
				if (!ensure(MeshMaterials.Num() > 0))
				{
					OutMaterialIndex = -1;
					return NAME_None;
				}
				if (PerSectionMaterialImportName.IsValidIndex(SectionIndex))
				{
					for (int32 MaterialIndex = 0; MaterialIndex < MeshMaterials.Num(); ++MaterialIndex)
					{
						const FSkeletalMaterial& Material = MeshMaterials[MaterialIndex];
						if (PerSectionMaterialImportName[SectionIndex] == Material.ImportedMaterialSlotName)
						{
							OutMaterialIndex = MaterialIndex;
							break;
						}
					}
					return PerSectionMaterialImportName[SectionIndex];
				}

				OutMaterialIndex = Section.MaterialIndex;
				if (LodInfo->LODMaterialMap.IsValidIndex(SectionIndex) && LodInfo->LODMaterialMap[SectionIndex] != INDEX_NONE)
				{
					OutMaterialIndex = LodInfo->LODMaterialMap[SectionIndex];
				}
				FName ImportedMaterialSlotName = NAME_None;
				if (MeshMaterials.IsValidIndex(OutMaterialIndex))
				{
					ImportedMaterialSlotName = MeshMaterials[OutMaterialIndex].ImportedMaterialSlotName;
				}
				else
				{
					ImportedMaterialSlotName = MeshMaterials[0].ImportedMaterialSlotName;
					OutMaterialIndex = 0;
				}
				return ImportedMaterialSlotName;
			};

		FSkeletalMeshLODModel& ExistLODModel = Parameters.SkeletalMesh->GetImportedModel()->LODModels[Parameters.LodIndex];
		for (int32 NewSectionIndex = 0; NewSectionIndex < ImportedMaterials.Num(); NewSectionIndex++)
		{
			FName ImportedMaterialName = *(ImportedMaterials[NewSectionIndex].MaterialImportName);
			FSkelMeshSourceSectionUserData& SourceSectionUserData = NewUserSectionsDataMap.FindOrAdd(ImportedMaterialName);

			for (int32 ExistSectionIndex = 0; ExistSectionIndex < ExistLODModel.Sections.Num(); ExistSectionIndex++)
			{
				if (ExistLODModel.Sections[ExistSectionIndex].ChunkedParentSectionIndex != INDEX_NONE)
				{
					continue;
				}

				const FSkelMeshSection& ExistingSection = ExistLODModel.Sections[ExistSectionIndex];
				//Skip chunked section
				if (ExistingSection.ChunkedParentSectionIndex != INDEX_NONE)
					continue;
				int32 ExistingMaterialIndex = 0;
				FName ExistingImportedMaterialSlotName = GetImportMaterialSlotName(ExistingSection
					, ExistingSection.OriginalDataSectionIndex
					, ExistingMaterialIndex
					, ExistingOriginalPerSectionMaterialImportName);

				TArray<FName> EmptyArray;
				int32 ExistingCurrentMaterialIndex = 0;
				FName ExistingCurrentImportedMaterialSlotName = GetImportMaterialSlotName(ExistingSection
					, ExistingSection.OriginalDataSectionIndex
					, ExistingCurrentMaterialIndex
					, EmptyArray);

				auto CopySectionData = [&ExistingSection, &SourceSectionUserData]()
					{
						//Set the user section data to reflect the existing settings
						SourceSectionUserData.bCastShadow = ExistingSection.bCastShadow;
						SourceSectionUserData.bVisibleInRayTracing = ExistingSection.bVisibleInRayTracing;
						SourceSectionUserData.bRecomputeTangent = ExistingSection.bRecomputeTangent;
						SourceSectionUserData.RecomputeTangentsVertexMaskChannel = ExistingSection.RecomputeTangentsVertexMaskChannel;
						SourceSectionUserData.bDisabled = ExistingSection.bDisabled;
						SourceSectionUserData.GenerateUpToLodIndex = ExistingSection.GenerateUpToLodIndex;
					};

				if (ExistingImportedMaterialSlotName != NAME_None)
				{
					if (ImportedMaterialName == ExistingImportedMaterialSlotName)
					{
						CopySectionData();
						//If user has edit the material slot pointer, store the data so we can modify the LODMaterialMap
						if (ExistingCurrentImportedMaterialSlotName != ExistingImportedMaterialSlotName)
						{
							LODMaterialMapRedirection.FindOrAdd(ImportedMaterialName) = ExistingCurrentMaterialIndex;
						}
						break;
					}
				}
				else if (Parameters.SkeletalMesh->GetMaterials().IsValidIndex(ExistingCurrentMaterialIndex) &&
					ImportedMaterials[NewSectionIndex].Material.Get() == Parameters.SkeletalMesh->GetMaterials()[ExistingCurrentMaterialIndex].MaterialInterface.Get()) //Use material slot compare to match in case the name is none
				{
					CopySectionData();
					break;
				}
			}
		}
	}

	LodInfo->LODMaterialMap.Empty();
	// Now set up the material mapping array.
	if (Parameters.LodIndex != 0)
	{
		for (int32 ImportedMaterialIndex = 0; ImportedMaterialIndex < ImportedMaterials.Num(); ImportedMaterialIndex++)
		{
			int32 LODMatIndex = INDEX_NONE;
			FName ImportedMaterialName = *(ImportedMaterials[ImportedMaterialIndex].MaterialImportName);

			//If the current lod section material slot was edited, we must keep the edition.
			if (int32* MaterialRedirection = LODMaterialMapRedirection.Find(ImportedMaterialName))
			{
				LODMatIndex = *MaterialRedirection;
			}
			else
			{
				//Match by name
				for (int32 MaterialIndex = 0; MaterialIndex < Materials.Num(); ++MaterialIndex)
				{
					const FSkeletalMaterial& SkeletalMaterial = Materials[MaterialIndex];
					if (SkeletalMaterial.ImportedMaterialSlotName != NAME_None && SkeletalMaterial.ImportedMaterialSlotName == ImportedMaterialName)
					{
						LODMatIndex = MaterialIndex;
						break;
					}
				}

				// If we don't have a match, add a new entry to the material list.
				if (LODMatIndex == INDEX_NONE)
				{
					LODMatIndex = Materials.Add(FSkeletalMaterial(ImportedMaterials[ImportedMaterialIndex].Material.Get(), true, false, ImportedMaterialName, ImportedMaterialName));
				}
			}
			LodInfo->LODMaterialMap.Add(LODMatIndex);
		}
	}
	else if (LODMaterialMapRedirection.Num() > 0)
	{
		for (int32 ExistMaterialIndex = 0; ExistMaterialIndex < Materials.Num(); ExistMaterialIndex++)
		{
			int32 LODMatIndex = INDEX_NONE;
			FName ExistMaterialName = Materials[ExistMaterialIndex].ImportedMaterialSlotName;
			//If the current lod section material slot was edited, we must keep the edition.
			if (int32* MaterialRedirection = LODMaterialMapRedirection.Find(ExistMaterialName))
			{
				LODMatIndex = *MaterialRedirection;
			}
			LodInfo->LODMaterialMap.Add(LODMatIndex);
		}
	}

	if (Parameters.bIsReImport)
	{
		FSkeletalMeshLODModel& ExistLODModel = Parameters.SkeletalMesh->GetImportedModel()->LODModels[Parameters.LodIndex];
		for (int32 NewSectionIndex = 0; NewSectionIndex < ImportedMaterials.Num(); NewSectionIndex++)
		{
			FName ImportedMaterialName = *(ImportedMaterials[NewSectionIndex].MaterialImportName);
			if (!NewUserSectionsDataMap.Contains(ImportedMaterialName))
			{
				NewUserSectionsDataMap.Add(ImportedMaterialName, FSkelMeshSourceSectionUserData());
			}
		}

		if (Parameters.LodIndex == 0)
		{
			//Since LOD 0 data will be re-arrange to follow the Material list order, sort the map accordingly.
			NewUserSectionsDataMap.KeySort([&Materials](const FName& A, const FName& B)
				{
					int32 MatAIndex = INDEX_NONE;
					int32 MatBIndex = INDEX_NONE;
					for (int32 MaterialIndex = 0; MaterialIndex < Materials.Num(); ++MaterialIndex)
					{
						if (Materials[MaterialIndex].ImportedMaterialSlotName == A)
						{
							MatAIndex = MaterialIndex;
						}
						if (Materials[MaterialIndex].ImportedMaterialSlotName == B)
						{
							MatBIndex = MaterialIndex;
						}
					}
					if (MatBIndex == INDEX_NONE)
					{
						return true;
					}
					if (MatAIndex == INDEX_NONE)
					{
						return false;
					}
					return MatAIndex < MatBIndex;
				});
		}

		//If the caller provide a CustomImportedLODModel we must apply the user section data to the provided lod model.
		if (Parameters.CustomImportedLODModel)
		{
			Parameters.CustomImportedLODModel->UserSectionsData.Reset();
			int32 RemapSectionIndex = 0;
			for (TPair<FName, FSkelMeshSourceSectionUserData> UserSectionsData : NewUserSectionsDataMap)
			{
				Parameters.CustomImportedLODModel->UserSectionsData.Add(RemapSectionIndex++, UserSectionsData.Value);
			}
			Parameters.CustomImportedLODModel->SyncronizeUserSectionsDataArray(true);
		}
		else
		{
			ExistLODModel.UserSectionsData.Reset();
			TBitArray<> MatchedMaterial;
			MatchedMaterial.Init(false, Materials.Num());
			TSet<FName> MatchedUserSections;
			MatchedUserSections.Reserve(NewUserSectionsDataMap.Num());
			for (TPair<FName, FSkelMeshSourceSectionUserData> UserSectionsData : NewUserSectionsDataMap)
			{
				for (int32 MaterialIndex = 0; MaterialIndex < Materials.Num(); ++MaterialIndex)
				{
					if (Materials[MaterialIndex].ImportedMaterialSlotName == UserSectionsData.Key)
					{
						MatchedMaterial[MaterialIndex] = true;
						MatchedUserSections.Add(UserSectionsData.Key);
						ExistLODModel.UserSectionsData.Add(MaterialIndex, UserSectionsData.Value);
						break;
					}
				}
			}
			//Add any missing user section data
			for (TPair<FName, FSkelMeshSourceSectionUserData> UserSectionsData : NewUserSectionsDataMap)
			{
				if (!MatchedUserSections.Contains(UserSectionsData.Key))
				{
					for (int32 MaterialIndex = 0; MaterialIndex < Materials.Num(); ++MaterialIndex)
					{
						if (!MatchedMaterial[MaterialIndex])
						{
							MatchedMaterial[MaterialIndex] = true;
							ExistLODModel.UserSectionsData.Add(MaterialIndex, UserSectionsData.Value);
							break;
						}
					}
				}
			}
			//Reorder the user section from the keys
			ExistLODModel.UserSectionsData.KeySort([](const int32& A, const int32& B)
				{
					return A < B;
				});
		}
	}
}

void FLODUtilities::ReorderMaterialSlotToBaseLod(USkeletalMesh* SkeletalMesh)
{
	//Make sure we have valid parameters
	if (!ensure(SkeletalMesh))
	{
		UE_ASSET_LOG(LogLODUtilities, Warning, SkeletalMesh, TEXT("FLODUtilities::ReorderMaterialSlotToBaseLod: Bad parameters, SkeletalMesh is null"));
		return;
	}
	TArray<FSkeletalMaterial>& Materials = SkeletalMesh->GetMaterials();
	if (Materials.Num() < 2)
	{
		return;
	}

	TArray<int32> MaterialSlotRemap;
	MaterialSlotRemap.AddUninitialized(Materials.Num());
	for (int32 MaterialIndex = 0; MaterialIndex < Materials.Num(); ++MaterialIndex)
	{
		MaterialSlotRemap[MaterialIndex] = INDEX_NONE;
	}
	TArray<FSkeletalMaterial> ReorderMaterialArray;
	ReorderMaterialArray.Reserve(Materials.Num());

	//Unique imported material list for all lods
	TArray<FName> AllLodSortedImportedMaterials;
	AllLodSortedImportedMaterials.Reserve(Materials.Num());

	TBitArray<> MatchedMaterials;
	MatchedMaterials.Init(false, Materials.Num());

	for (int32 LodIndex = 0; LodIndex < SkeletalMesh->GetLODNum(); ++LodIndex)
	{
		if (!SkeletalMesh->HasMeshDescription(LodIndex))
		{
			if (LodIndex == 0)
			{
				UE_ASSET_LOG(LogLODUtilities, Warning, SkeletalMesh, TEXT("FLODUtilities::ReorderMaterialSlotToBaseLod: Skeletal mesh invalid import data for LOD 0"));
				return;
			}
			continue;
		}

		FSkeletalMeshImportData LodImportData;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		SkeletalMesh->LoadLODImportedData(LodIndex, LodImportData);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		const TArray<SkeletalMeshImportData::FMaterial>& ImportedMaterials = LodImportData.Materials;

		for (int32 ImportedMaterialIndex = 0; ImportedMaterialIndex < ImportedMaterials.Num(); ++ImportedMaterialIndex)
		{
			FName ImportedMaterialSlotName = FName(*ImportedMaterials[ImportedMaterialIndex].MaterialImportName);
			//add Unique Material
			if (!AllLodSortedImportedMaterials.Contains(ImportedMaterialSlotName))
			{
				AllLodSortedImportedMaterials.Add(ImportedMaterialSlotName);
				//Make sure matched material is filled
				for (int32 MaterialIndex = 0; MaterialIndex < Materials.Num(); ++MaterialIndex)
				{
					FName MaterialSlotName = Materials[MaterialIndex].ImportedMaterialSlotName;
					if (MaterialSlotName == ImportedMaterialSlotName)
					{
						MatchedMaterials[MaterialIndex] = true;
						break;
					}
				}
			}
		}
	}

	//Now fill the ReorderMaterialArray using the AllLodSortedImportedMaterials order
	for (int32 ImportedMaterialIndex = 0; ImportedMaterialIndex < AllLodSortedImportedMaterials.Num(); ++ImportedMaterialIndex)
	{
		FName ImportedMaterialSlotName = AllLodSortedImportedMaterials[ImportedMaterialIndex];
		int32 TargetMaterialIndex = INDEX_NONE;
		for (int32 MaterialIndex = 0; MaterialIndex < Materials.Num(); ++MaterialIndex)
		{
			//If we have a match we should find back the material index and assign it to the target material index
			if (MatchedMaterials[MaterialIndex])
			{
				FName MaterialSlotName = Materials[MaterialIndex].ImportedMaterialSlotName;
				if (MaterialSlotName == ImportedMaterialSlotName)
				{
					TargetMaterialIndex = MaterialIndex;
					break;
				}
			}
		}
		//If we have an unmatched imported material we use the next unmatched material in the original list
		if (TargetMaterialIndex == INDEX_NONE)
		{
			for (int32 MaterialIndex = 0; MaterialIndex < Materials.Num(); ++MaterialIndex)
			{
				if (MatchedMaterials[MaterialIndex])
				{
					continue;
				}
				ensure(MaterialSlotRemap[MaterialIndex] == INDEX_NONE);
				TargetMaterialIndex = MaterialIndex;
				MatchedMaterials[MaterialIndex] = true;
				break;
			}
		}
		//We should have found a valid material index at this point
		if (ensure(MaterialSlotRemap.IsValidIndex(TargetMaterialIndex)))
		{
			if (MaterialSlotRemap[TargetMaterialIndex] == INDEX_NONE)
			{
				MaterialSlotRemap[TargetMaterialIndex] = ReorderMaterialArray.Add(Materials[TargetMaterialIndex]);
			}
		}
	}

	//Unmatched material are appended to the end of the material slots
	for (int32 MaterialIndex = 0; MaterialIndex < Materials.Num(); ++MaterialIndex)
	{
		if (MaterialSlotRemap[MaterialIndex] == INDEX_NONE)
		{
			MaterialSlotRemap[MaterialIndex] = ReorderMaterialArray.Add(Materials[MaterialIndex]);
		}
	}

	check(ReorderMaterialArray.Num() == Materials.Num());

	//Reorder the skeletal mesh material slot array
	SkeletalMesh->SetMaterials(ReorderMaterialArray);
	
	//We now need to adjust all LODs data to fit the re-order
	for (int32 LodIndex = 0; LodIndex < SkeletalMesh->GetLODNum(); ++LodIndex)
	{
		if (ensure(SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(LodIndex)))
		{
			FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[LodIndex];
			TMap<int32, FSkelMeshSourceSectionUserData> RemapSectionsDataMap;
			for (TPair<int32, FSkelMeshSourceSectionUserData>& UserSectionDataPair : LODModel.UserSectionsData)
			{
				RemapSectionsDataMap.Add(MaterialSlotRemap[UserSectionDataPair.Key], UserSectionDataPair.Value);
			}
			//Sort the remap section so its in the proper order. Optional but easier to debug ordered data
			RemapSectionsDataMap.KeySort([](const int32& A, const int32& B)
				{
					return A < B;
				});

			//Rebuild a remap with the new order starting from 0 to follow the section order
			LODModel.UserSectionsData.Empty(RemapSectionsDataMap.Num());
			int32 KeyIndex = 0;
			for (TPair<int32, FSkelMeshSourceSectionUserData>& UserSectionDataPair : RemapSectionsDataMap)
			{
				LODModel.UserSectionsData.Add(KeyIndex++, UserSectionDataPair.Value);
			}
		}
		else
		{
			UE_ASSET_LOG(LogLODUtilities, Warning, SkeletalMesh, TEXT("FLODUtilities::ReorderMaterialSlotToBaseLod: Skeletal mesh invalid LODModel %d"), LodIndex);
		}
		
		//Remap the LODMaterialMap
		if (FSkeletalMeshLODInfo* LodInfo = SkeletalMesh->GetLODInfo(LodIndex))
		{
			for (int32& MaterialMap : LodInfo->LODMaterialMap)
			{
				if (MaterialSlotRemap.IsValidIndex(MaterialMap))
				{
					MaterialMap = MaterialSlotRemap[MaterialMap];
				}
			}
		}
		else
		{
			UE_ASSET_LOG(LogLODUtilities, Warning, SkeletalMesh, TEXT("FLODUtilities::ReorderMaterialSlotToBaseLod: Skeletal mesh invalid LODInfo %d"), LodIndex);
		}
	}
}

void FLODUtilities::RemoveUnusedMaterialSlot(USkeletalMesh* SkeletalMesh)
{
	if (!SkeletalMesh || !SkeletalMesh->HasMeshDescription(0))
	{
		return;
	}

	TArray<FSkeletalMaterial>& Materials = SkeletalMesh->GetMaterials();
	if (Materials.Num() < 2)
	{
		return;
	}
	FSkeletalMeshModel* ImportedResource = SkeletalMesh->GetImportedModel();

	TArray<int32> UsedIndexes;
	for (int32 LodIndex = 0, LODNum = SkeletalMesh->GetLODNum(); LodIndex < LODNum; ++LodIndex)
	{
		const TArray<int32>& LODMaterialMap = SkeletalMesh->GetLODInfo(LodIndex)->LODMaterialMap;
		if (!LODMaterialMap.IsEmpty())
		{
			for (int32 MaterialIndex : LODMaterialMap)
			{
				if (MaterialIndex != INDEX_NONE)
				{
					UsedIndexes.AddUnique(MaterialIndex);
				}
			}
		}

		if (LodIndex == 0 && ImportedResource && ImportedResource->LODModels.IsValidIndex(LodIndex))
		{
			//Since LOD 0 is always reordering the material slot array, we expect section index to match UserSectionsData order
			//We cannot use the section since section are built by the DDC cache.
			FSkeletalMeshLODModel& Model = ImportedResource->LODModels[LodIndex];
			for (TPair<int32, FSkelMeshSourceSectionUserData> UserSectionsData : Model.UserSectionsData)
			{
				if (LODMaterialMap.IsEmpty() || !LODMaterialMap.IsValidIndex(UserSectionsData.Key) || LODMaterialMap[UserSectionsData.Key] == INDEX_NONE)
				{
					UsedIndexes.AddUnique(UserSectionsData.Key);
				}
			}
		}
	}

	//Clean up the trailing unused material
	for (int32 MaterialIndex = Materials.Num() - 1; MaterialIndex >= 0; MaterialIndex--)
	{
		if (UsedIndexes.Contains(MaterialIndex))
		{
			//Only delete extra material at the end of the material slot list, because MaterialMap and UserSectionData will need to be patch otherwise
			break;
		}
		Materials.RemoveAt(MaterialIndex);
	}
}

namespace TriangleStripHelper
{
	struct FTriangle2D
	{
		FVector2D Vertices[3];
	};

	bool IntersectTriangleAndAABB(const FTriangle2D& Triangle, const FBox2D& Box)
	{
		FBox2D TriangleBox(Triangle.Vertices[0], Triangle.Vertices[0]);
		TriangleBox += Triangle.Vertices[1];
		TriangleBox += Triangle.Vertices[2];

		auto IntersectBoxes = [&TriangleBox, &Box]()-> bool
		{
			if ((FMath::RoundToInt(TriangleBox.Min.X) >= FMath::RoundToInt(Box.Max.X)) || (FMath::RoundToInt(Box.Min.X) >= FMath::RoundToInt(TriangleBox.Max.X)))
			{
				return false;
			}

			if ((FMath::RoundToInt(TriangleBox.Min.Y) >= FMath::RoundToInt(Box.Max.Y)) || (FMath::RoundToInt(Box.Min.Y) >= FMath::RoundToInt(TriangleBox.Max.Y)))
			{
				return false;
			}

			return true;
		};

		//If the triangle box do not intersect, return false
		if (!IntersectBoxes())
		{
			return false;
		}

		auto IsInsideBox = [&Box](const FVector2D& TestPoint)->bool
		{
			return ((FMath::RoundToInt(TestPoint.X) >= FMath::RoundToInt(Box.Min.X)) &&
					(FMath::RoundToInt(TestPoint.X) <= FMath::RoundToInt(Box.Max.X)) &&
					(FMath::RoundToInt(TestPoint.Y) >= FMath::RoundToInt(Box.Min.Y)) &&
					(FMath::RoundToInt(TestPoint.Y) <= FMath::RoundToInt(Box.Max.Y)) );
		};

		if( IsInsideBox(Triangle.Vertices[0]) ||
			IsInsideBox(Triangle.Vertices[1]) ||
			IsInsideBox(Triangle.Vertices[2]) )
		{
			return true;
		}

		auto SegmentIntersection2D = [](const FVector2D & SegmentStartA, const FVector2D & SegmentEndA, const FVector2D & SegmentStartB, const FVector2D & SegmentEndB)
		{
			const FVector2D VectorA = SegmentEndA - SegmentStartA;
			const FVector2D VectorB = SegmentEndB - SegmentStartB;

			const double S = (-VectorA.Y * (SegmentStartA.X - SegmentStartB.X) + VectorA.X * (SegmentStartA.Y - SegmentStartB.Y)) / (-VectorB.X * VectorA.Y + VectorA.X * VectorB.Y);
			const double T = (VectorB.X * (SegmentStartA.Y - SegmentStartB.Y) - VectorB.Y * (SegmentStartA.X - SegmentStartB.X)) / (-VectorB.X * VectorA.Y + VectorA.X * VectorB.Y);

			return (S >= 0 && S <= 1 && T >= 0 && T <= 1);
		};

		auto IsInsideTriangle = [&Triangle, &SegmentIntersection2D, &Box, &TriangleBox](const FVector2D& TestPoint)->bool
		{
			double Extent = (2.0 * Box.GetSize().Size()) + (2.0 * TriangleBox.GetSize().Size());
			FVector2D TestPointExtend(Extent, Extent);
			int32 IntersectionCount = SegmentIntersection2D(Triangle.Vertices[0], Triangle.Vertices[1], TestPoint, TestPoint + TestPointExtend) ? 1 : 0;
			IntersectionCount += SegmentIntersection2D(Triangle.Vertices[1], Triangle.Vertices[2], TestPoint, TestPoint + TestPointExtend) ? 1 : 0;
			IntersectionCount += SegmentIntersection2D(Triangle.Vertices[2], Triangle.Vertices[0], TestPoint, TestPoint + TestPointExtend) ? 1 : 0;
			return (IntersectionCount == 1);
		};
	
		if (IsInsideTriangle(Box.Min) ||
			IsInsideTriangle(Box.Max) ||
			IsInsideTriangle(FVector2D(Box.Min.X, Box.Max.Y)) ||
			IsInsideTriangle(FVector2D(Box.Max.X, Box.Min.Y)))
		{
			return true;
		}

		auto IsTriangleEdgeIntersectBoxEdges = [&SegmentIntersection2D, &Box]( const FVector2D& EdgeStart, const FVector2D& EdgeEnd)->bool
		{
			//Triangle Edges 0-1 intersection with box
			if( SegmentIntersection2D(EdgeStart, EdgeEnd, Box.Min, FVector2D(Box.Min.X, Box.Max.Y)) ||
				SegmentIntersection2D(EdgeStart, EdgeEnd, Box.Max, FVector2D(Box.Min.X, Box.Max.Y)) ||
				SegmentIntersection2D(EdgeStart, EdgeEnd, Box.Max, FVector2D(Box.Max.X, Box.Min.Y)) ||
				SegmentIntersection2D(EdgeStart, EdgeEnd, Box.Min, FVector2D(Box.Max.X, Box.Min.Y)) )
			{
				return true;
			}
			return false;
		};

		if( IsTriangleEdgeIntersectBoxEdges(Triangle.Vertices[0], Triangle.Vertices[1]) ||
			IsTriangleEdgeIntersectBoxEdges(Triangle.Vertices[1], Triangle.Vertices[2]) || 
			IsTriangleEdgeIntersectBoxEdges(Triangle.Vertices[2], Triangle.Vertices[0]))
		{
			return true;
		}
		return false;
	}
} //End namespace TriangleStripHelper

bool FLODUtilities::StripLODGeometry(USkeletalMesh* SkeletalMesh, const int32 LODIndex, UTexture2D* TextureMask, const float Threshold)
{
	if (LODIndex < 0 || LODIndex >= SkeletalMesh->GetLODNum() || !SkeletalMesh->GetImportedModel() || !SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex) || !TextureMask)
	{
		UE_LOGF(LogLODUtilities, Warning, "Cannot strip triangle for skeletalmesh %ls LOD %d.", *SkeletalMesh->GetPathName(), LODIndex);
		return false;
	}
	
	//Grab the reference data
	FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[LODIndex];
	const FSkeletalMeshLODInfo& LODInfo = *(SkeletalMesh->GetLODInfo(LODIndex));
	const bool bIsReductionActive = SkeletalMesh->IsReductionActive(LODIndex);
	if (bIsReductionActive && LODInfo.ReductionSettings.BaseLOD < LODIndex)
	{
		//No need to strip if the LOD is reduce using another LOD as the source data
		UE_LOGF(LogLODUtilities, Warning, "Cannot strip triangle for skeletalmesh %ls LOD %d. Because this LOD is generated, strip the source instead.", *SkeletalMesh->GetPathName(), LODIndex);
		return false;
	}

	//Check the texture mask source data, it must be valid
	FTextureSource& InitialSource = TextureMask->Source;
	const int32 ResX = InitialSource.GetSizeX();
	const int32 ResY = InitialSource.GetSizeY();
	const int32 FormatDataSize = InitialSource.GetBytesPerPixel();
	if (FormatDataSize <= 0)
	{
		UE_LOGF(LogLODUtilities, Warning, "Cannot strip triangle for skeletalmesh %ls LOD %d. Because the texture format size is 0.", *SkeletalMesh->GetPathName(), LODIndex);
		return false;
	}
	ETextureSourceFormat SourceFormat = InitialSource.GetFormat();
	if (SourceFormat <= TSF_Invalid || SourceFormat >= TSF_MAX)
	{
		UE_LOGF(LogLODUtilities, Warning, "Cannot strip triangle for skeletalmesh %ls LOD %d. Because the texture format is invalid.", *SkeletalMesh->GetPathName(), LODIndex);
		return false;
	}
	TArray64<uint8> Ref2DData;
	if (!InitialSource.GetMipData(Ref2DData, 0, nullptr))
	{
		UE_LOGF(LogLODUtilities, Warning, "Cannot strip triangle for skeletalmesh %ls LOD %d. Because the texture data cannot be extracted.", *SkeletalMesh->GetPathName(), LODIndex);
		return false;
	}

	//Post edit change scope
	{
		FScopedSkeletalMeshPostEditChange ScopePostEditChange(SkeletalMesh);
		//This is like a re-import, we must force to use a new DDC
		SkeletalMesh->InvalidateDeriveDataCacheGUID();
		const bool bBuildAvailable = SkeletalMesh->HasMeshDescription(LODIndex);
		FSkeletalMeshImportData ImportedData;
		//Get the imported data if available
		if (bBuildAvailable)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			SkeletalMesh->LoadLODImportedData(LODIndex, ImportedData);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		SkeletalMesh->Modify();
		
		ERawImageFormat::Type RawFormat = FImageCoreUtils::ConvertToRawImageFormat(SourceFormat);
		bool bSRGB = TextureMask->SRGB;

		auto ShouldStripTriangle = [&](const FVector2D& UvA, const FVector2D& UvB, const FVector2D& UvC)->bool
		{
			const FIntVector2 PixelUvA(FMath::FloorToInt32(UvA.X * ResX) % (ResX + 1), FMath::FloorToInt32(UvA.Y * ResY) % (ResY + 1));
			const FIntVector2 PixelUvB(FMath::FloorToInt32(UvB.X * ResX) % (ResX + 1), FMath::FloorToInt32(UvB.Y * ResY) % (ResY + 1));
			const FIntVector2 PixelUvC(FMath::FloorToInt32(UvC.X * ResX) % (ResX + 1), FMath::FloorToInt32(UvC.Y * ResY) % (ResY + 1));

			const int32 MinU = FMath::Clamp(FMath::Min3<int32>(PixelUvA.X, PixelUvB.X, PixelUvC.X), 0, ResX);
			const int32 MinV = FMath::Clamp(FMath::Min3<int32>(PixelUvA.Y, PixelUvB.Y, PixelUvC.Y), 0, ResY);
			const int32 MaxU = FMath::Clamp(FMath::Max3<int32>(PixelUvA.X, PixelUvB.X, PixelUvC.X), 0, ResX);
			const int32 MaxV = FMath::Clamp(FMath::Max3<int32>(PixelUvA.Y, PixelUvB.Y, PixelUvC.Y), 0, ResY);

			//Do not read the alpha value when testing the texture value
			auto IsPixelZero = [&](int32 PosX, int32 PosY) -> bool
			{
				const int32 RefPos = PosX + (PosY * InitialSource.GetSizeX());
				const void * PixelPtr = Ref2DData.GetData() + RefPos * FormatDataSize;
				
				const FLinearColor Color = ERawImageFormat::GetOnePixelLinear(PixelPtr,RawFormat,bSRGB);	

				const bool bPixelIsZero = 
					FMath::IsNearlyZero(Color.R,Threshold) &&
					FMath::IsNearlyZero(Color.G,Threshold) &&
					FMath::IsNearlyZero(Color.B,Threshold);

				return bPixelIsZero;
			};

			//Triangle smaller or equal to one pixel just need to test the pixel color value
			if (MinU == MaxU || MinV == MaxV)
			{
				return IsPixelZero(MinU, MinV);
			}

			for (int32 PosY = MinV; PosY < MaxV; ++PosY)
			{
				for (int32 PosX = MinU; PosX < MaxU; ++PosX)
				{
					const bool bStripPixel = IsPixelZero(PosX, PosY);

					//if any none zeroed pixel intersect the triangle, prevent stripping of this triangle
					if (!bStripPixel)
					{
						FVector2D StartPixel(PosX, PosY);
						FVector2D EndPixel(PosX + 1, PosY + 1);
						FBox2D Box2D(StartPixel, EndPixel);
						//Test if the triangle UV touch this pixel
						TriangleStripHelper::FTriangle2D Triangle;
						Triangle.Vertices[0] = FVector2D(PixelUvA.X, PixelUvA.Y);
						Triangle.Vertices[1] = FVector2D(PixelUvB.X, PixelUvB.Y);
						Triangle.Vertices[2] = FVector2D(PixelUvC.X, PixelUvC.Y);
						if(TriangleStripHelper::IntersectTriangleAndAABB(Triangle, Box2D))
						{
							return false;
						}
					}
				}
			}
			return true;
		};

		const TArray< uint32 >& SoftVertexIndexToImportDataPointIndex = LODModel.GetRawPointIndices();

		TMap<FInt32Vector3, int32> OptimizedFaceFinder;

		for (int32 FaceIndex = 0; FaceIndex < ImportedData.Faces.Num(); ++FaceIndex)
		{
			const SkeletalMeshImportData::FTriangle& Face = ImportedData.Faces[FaceIndex];
			int32 FaceVertexA = ImportedData.Wedges[Face.WedgeIndex[0]].VertexIndex;
			int32 FaceVertexB = ImportedData.Wedges[Face.WedgeIndex[1]].VertexIndex;
			int32 FaceVertexC = ImportedData.Wedges[Face.WedgeIndex[2]].VertexIndex;

			OptimizedFaceFinder.Add(FInt32Vector3(FaceVertexA, FaceVertexB, FaceVertexC), FaceIndex);
		}

		int32 RemovedFaceCount = 0;
		TBitArray<> FaceToRemove;
		FaceToRemove.Init(false, ImportedData.Faces.Num());
		int32 NumTriangleIndex = LODModel.IndexBuffer.Num();
		for (int32 TriangleIndex = NumTriangleIndex - 1; TriangleIndex >= 0; TriangleIndex -= 3)
		{
			int32 VertexIndexA = LODModel.IndexBuffer[TriangleIndex - 2];
			int32 VertexIndexB = LODModel.IndexBuffer[TriangleIndex - 1];
			int32 VertexIndexC = LODModel.IndexBuffer[TriangleIndex];
			int32 SectionIndex;
			int32 SectionVertexIndexA;
			int32 SectionVertexIndexB;
			int32 SectionVertexIndexC;
			LODModel.GetSectionFromVertexIndex(VertexIndexA, SectionIndex, SectionVertexIndexA);
			LODModel.GetSectionFromVertexIndex(VertexIndexB, SectionIndex, SectionVertexIndexB);
			LODModel.GetSectionFromVertexIndex(VertexIndexC, SectionIndex, SectionVertexIndexC);
			FSkelMeshSection& Section = LODModel.Sections[SectionIndex];
			//Get the UV triangle, add the small number that will act like threshold when converting the UV into pixel coordinate.
			FVector2D UvA = FVector2D(Section.SoftVertices[SectionVertexIndexA].UVs[0]) + KINDA_SMALL_NUMBER;
			FVector2D UvB = FVector2D(Section.SoftVertices[SectionVertexIndexB].UVs[0]) + KINDA_SMALL_NUMBER;
			FVector2D UvC = FVector2D(Section.SoftVertices[SectionVertexIndexC].UVs[0]) + KINDA_SMALL_NUMBER;

			if (ShouldStripTriangle(UvA, UvB, UvC))
			{
				//Find the face in the imported data
				if (bBuildAvailable)
				{
					//Findback the face in the import data
					int32 ImportedPointIndexA = SoftVertexIndexToImportDataPointIndex[VertexIndexA];
					int32 ImportedPointIndexB = SoftVertexIndexToImportDataPointIndex[VertexIndexB];
					int32 ImportedPointIndexC = SoftVertexIndexToImportDataPointIndex[VertexIndexC];
					int32 FaceIndex = OptimizedFaceFinder.FindChecked(FInt32Vector3(ImportedPointIndexA, ImportedPointIndexB, ImportedPointIndexC));
					if (!FaceToRemove[FaceIndex])
					{
						FaceToRemove[FaceIndex] = true;
						RemovedFaceCount++;
					}
				}
				else
				{
					//Remove the source model vertex if there is no build data
					LODModel.IndexBuffer.RemoveAt(TriangleIndex - 2, 3, EAllowShrinking::No);
				}
			}
		}
		
		if(bBuildAvailable && RemovedFaceCount > 0)
		{
			//Recreate a new imported data with only the remaining faces
			FSkeletalMeshImportData StrippedImportedData;
			StrippedImportedData = ImportedData;
			StrippedImportedData.Faces.Reset();
			StrippedImportedData.Wedges.Reset();
			StrippedImportedData.Points.Reset();
			StrippedImportedData.PointToRawMap.Reset();
			StrippedImportedData.Influences.Reset();

			TArray<int32> RemapVertexIndex;
			RemapVertexIndex.AddZeroed(ImportedData.Points.Num());
			for (int32 VertexIndex = 0; VertexIndex < ImportedData.Points.Num(); ++VertexIndex)
			{
				RemapVertexIndex[VertexIndex] = INDEX_NONE;
			}

			StrippedImportedData.Faces.AddDefaulted(ImportedData.Faces.Num() - RemovedFaceCount);
			StrippedImportedData.Wedges.AddDefaulted(StrippedImportedData.Faces.Num()*3);
			int32 NewFaceIndex = 0;
			int32 NewWedgeIndex = 0;
			for (int32 FaceIndex = 0; FaceIndex < ImportedData.Faces.Num(); ++FaceIndex)
			{
				//Skip removed faces
				if (FaceToRemove[FaceIndex])
				{
					continue;
				}

				SkeletalMeshImportData::FTriangle& NewFace = StrippedImportedData.Faces[NewFaceIndex++];
				NewFace = ImportedData.Faces[FaceIndex];
				for(int32 FaceWedgeIndex = 0; FaceWedgeIndex < 3; ++FaceWedgeIndex)
				{
					SkeletalMeshImportData::FVertex& NewWedge = StrippedImportedData.Wedges[NewWedgeIndex];
					NewWedge = ImportedData.Wedges[NewFace.WedgeIndex[FaceWedgeIndex]];
					NewFace.WedgeIndex[FaceWedgeIndex] = NewWedgeIndex;
					int32 VertexIndex = NewWedge.VertexIndex;
					if(RemapVertexIndex[VertexIndex] == INDEX_NONE)
					{
						StrippedImportedData.PointToRawMap.Add(ImportedData.PointToRawMap[VertexIndex]);
						NewWedge.VertexIndex = StrippedImportedData.Points.Add(ImportedData.Points[VertexIndex]);
						RemapVertexIndex[VertexIndex] = NewWedge.VertexIndex;
					}
					else
					{
						NewWedge.VertexIndex = RemapVertexIndex[VertexIndex];
					}
					NewWedgeIndex++;
				}
			}
			
			//Fix the influences with the RemapVertexIndex
			for (int32 InfluenceIndex = 0; InfluenceIndex < ImportedData.Influences.Num(); ++InfluenceIndex)
			{
				int32 VertexIndex = ImportedData.Influences[InfluenceIndex].VertexIndex;
				int32 RemappedVertexIndex = RemapVertexIndex[VertexIndex];
				if(RemappedVertexIndex != INDEX_NONE)
				{
					SkeletalMeshImportData::FRawBoneInfluence& Influence = StrippedImportedData.Influences.Add_GetRef(ImportedData.Influences[InfluenceIndex]);
					Influence.VertexIndex = RemapVertexIndex[VertexIndex];
				}
			}
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			SkeletalMesh->SaveLODImportedData(LODIndex, StrippedImportedData);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
	return true;
}

#undef LOCTEXT_NAMESPACE // "LODUtilities"

#endif //WITH_EDITOR

