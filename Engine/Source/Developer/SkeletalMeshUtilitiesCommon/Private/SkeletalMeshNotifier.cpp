// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshNotifier.h"

FSkeletalMeshNotifyDelegate& ISkeletalMeshNotifier::Delegate()
{
	return NotifyDelegate;
}

bool ISkeletalMeshNotifier::Notifying() const
{
	return bNotifying;
}

void ISkeletalMeshNotifier::Notify(const TArray<FName>& BoneNames, const ESkeletalMeshNotifyType InNotifyType) const
{
	if (!bNotifying)
	{
		TGuardValue<bool> RecursionGuard(bNotifying, true);
		NotifyDelegate.Broadcast(BoneNames, InNotifyType);
	}
}

FSkeletalMeshNotifierBindScope::FSkeletalMeshNotifierBindScope(
	TWeakPtr<ISkeletalMeshNotifier> InNotifierA,
	TWeakPtr<ISkeletalMeshNotifier> InNotifierB
	) : NotifierA(InNotifierA)
	, NotifierB(InNotifierB)
{

	DelegateToRemoveFromNotifierA = InNotifierA.Pin()->Delegate().AddLambda(
	[InNotifierB](const TArray<FName>& BoneNames, const ESkeletalMeshNotifyType InNotifyType)
	{
		if (InNotifierB.IsValid())
		{
			InNotifierB.Pin()->HandleNotification(BoneNames, InNotifyType);
		}
	});
	
	DelegateToRemoveFromNotifierB = InNotifierB.Pin()->Delegate().AddLambda(
	[InNotifierA](const TArray<FName>& BoneNames, const ESkeletalMeshNotifyType InNotifyType)
	{
		if (InNotifierA.IsValid())
		{
			InNotifierA.Pin()->HandleNotification(BoneNames, InNotifyType);
		}
	});
}

FSkeletalMeshNotifierBindScope::~FSkeletalMeshNotifierBindScope()
{
	if (DelegateToRemoveFromNotifierA.IsValid() && NotifierA.IsValid())
	{
		verify(NotifierA.Pin()->Delegate().Remove(DelegateToRemoveFromNotifierA));
	}

	if (DelegateToRemoveFromNotifierB.IsValid() && NotifierB.IsValid())
	{
		verify(NotifierB.Pin()->Delegate().Remove(DelegateToRemoveFromNotifierB));
	}
}
