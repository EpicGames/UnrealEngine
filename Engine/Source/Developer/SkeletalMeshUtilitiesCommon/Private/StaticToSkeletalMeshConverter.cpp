// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticToSkeletalMeshConverter.h"

#if WITH_EDITOR

#include "Animation/Skeleton.h"
#include "EditorFramework/AssetImportData.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/StaticMesh.h"
#include "InterchangeHelper.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "LODUtilities.h"
#include "MeshDescription.h"
#include "MeshUtilities.h"
#include "Misc/Guid.h"
#include "Modules/ModuleManager.h"
#include "ReferenceSkeleton.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
#include "Rendering/SkeletalMeshModel.h"
#include "SkeletalMeshAttributes.h"


DEFINE_LOG_CATEGORY_STATIC(LogStaticToSkeletalMeshConverter, Log, All);

static const FName RootBoneName("Root");
static const TCHAR* JointBaseName(TEXT("Joint"));


bool FStaticToSkeletalMeshConverter::InitializeSkeletonFromStaticMesh(
	USkeleton* InSkeleton,
	const UStaticMesh* InStaticMesh,
	const FVector& InRelativeRootPosition
	)
{
	if (!ensure(InSkeleton))
	{
		return false;
	}
	
	if (InSkeleton->GetReferenceSkeleton().GetNum() != 0)
	{
		UE_LOGF(LogStaticToSkeletalMeshConverter, Error, "Skeleton '%ls' is not empty", *InSkeleton->GetPathName());
		return false;
	}
	
	if (!ensure(InStaticMesh))
	{
		return false;
	}

	const FBox Bounds = InStaticMesh->GetBoundingBox();
	const FVector RootPosition = Bounds.Min + (Bounds.Max - Bounds.Min) * InRelativeRootPosition;
	FTransform RootTransform(FTransform::Identity);
	RootTransform.SetTranslation(RootPosition);

	FReferenceSkeletonModifier Modifier(InSkeleton);
	Modifier.Add(FMeshBoneInfo(RootBoneName, RootBoneName.ToString(), INDEX_NONE), RootTransform);

	return true;
}


bool FStaticToSkeletalMeshConverter::InitializeSkeletonFromStaticMesh(
	USkeleton* InSkeleton,
	const UStaticMesh* InStaticMesh,
	const FVector& InRelativeRootPosition,
	const FVector& InRelativeEndEffectorPosition,
	const int32 InIntermediaryJointCount
	)
{
	if (!ensure(InSkeleton))
	{
		return false;
	}
	
	if (InSkeleton->GetReferenceSkeleton().GetNum() != 0)
	{
		UE_LOGF(LogStaticToSkeletalMeshConverter, Error, "Skeleton '%ls' is not empty", *InSkeleton->GetPathName());
		return false;
	}

	if (!ensure(InStaticMesh))
	{
		return false;
	}
	
	if (FMath::IsNearlyZero(FVector::DistSquared(InRelativeEndEffectorPosition, InRelativeRootPosition)))
	{
		return InitializeSkeletonFromStaticMesh(InSkeleton, InStaticMesh, InRelativeRootPosition);
	}

	const FBox Bounds = InStaticMesh->GetBoundingBox();
	const FVector RootPosition = Bounds.Min + (Bounds.Max - Bounds.Min) * InRelativeRootPosition;
	const FVector EndEffectorPosition = Bounds.Min + (Bounds.Max - Bounds.Min) * InRelativeEndEffectorPosition;

	// Find a rough rotation we can use
	const FQuat Rotation = FQuat::FindBetweenVectors(FVector::ZAxisVector, EndEffectorPosition - RootPosition).GetNormalized(); 
	
	FTransform ParentTransform(FTransform::Identity);
	ParentTransform.SetTranslation(RootPosition);
	ParentTransform.SetRotation(Rotation);

	FReferenceSkeletonModifier Modifier(InSkeleton);
	Modifier.Add(FMeshBoneInfo(RootBoneName, RootBoneName.ToString(), INDEX_NONE), ParentTransform);

	for (int32 JointIndex = 0; JointIndex <= InIntermediaryJointCount; JointIndex++)
	{
		const double T = (JointIndex + 1.0) / (InIntermediaryJointCount + 2.0);
		FTransform PointTransform(ParentTransform);
		PointTransform.SetTranslation(RootPosition + (EndEffectorPosition - RootPosition) * T);

		FString JointName = FString::Printf(TEXT("%s_%d"), JointBaseName, JointIndex + 1);
		Modifier.Add(FMeshBoneInfo(FName(JointName), JointName, JointIndex), PointTransform * ParentTransform.Inverse());
		ParentTransform = PointTransform;
	}

	return true;
}

static void CopyBuildSettings(
	const FMeshBuildSettings& InStaticMeshBuildSettings,
	FSkeletalMeshBuildSettings& OutSkeletalMeshBuildSettings
	)
{
	OutSkeletalMeshBuildSettings.bRecomputeNormals = InStaticMeshBuildSettings.bRecomputeNormals;
	OutSkeletalMeshBuildSettings.bRecomputeTangents = InStaticMeshBuildSettings.bRecomputeTangents;
	OutSkeletalMeshBuildSettings.bUseMikkTSpace = InStaticMeshBuildSettings.bUseMikkTSpace;
	OutSkeletalMeshBuildSettings.bComputeWeightedNormals = InStaticMeshBuildSettings.bComputeWeightedNormals;
	OutSkeletalMeshBuildSettings.bRemoveDegenerates = InStaticMeshBuildSettings.bRemoveDegenerates;
	OutSkeletalMeshBuildSettings.bUseHighPrecisionTangentBasis = InStaticMeshBuildSettings.bUseHighPrecisionTangentBasis;
	OutSkeletalMeshBuildSettings.bUseFullPrecisionUVs = InStaticMeshBuildSettings.bUseFullPrecisionUVs;
	OutSkeletalMeshBuildSettings.bUseBackwardsCompatibleF16TruncUVs = InStaticMeshBuildSettings.bUseBackwardsCompatibleF16TruncUVs;
	// The rest we leave at defaults.
}

static SkeletalMeshOptimizationImportance ConvertOptimizationImportance(
	 EMeshFeatureImportance::Type InStaticMeshImportance)
{
	switch(InStaticMeshImportance)
	{
	default:
	case EMeshFeatureImportance::Off:		return SMOI_Highest;
	case EMeshFeatureImportance::Lowest:	return SMOI_Lowest;
	case EMeshFeatureImportance::Low:		return SMOI_Low;
	case EMeshFeatureImportance::Normal:	return SMOI_Normal;
	case EMeshFeatureImportance::High:		return SMOI_High;
	case EMeshFeatureImportance::Highest:	return SMOI_Highest;
	}
}

static void CopyReductionSettings(
	const FMeshReductionSettings& InStaticMeshReductionSettings,
	FSkeletalMeshOptimizationSettings& OutSkeletalMeshReductionSettings
	)
{
	// Copy the reduction settings as closely as we can. 
	OutSkeletalMeshReductionSettings.NumOfTrianglesPercentage = InStaticMeshReductionSettings.PercentTriangles;
	OutSkeletalMeshReductionSettings.NumOfVertPercentage = InStaticMeshReductionSettings.PercentVertices;
	
	OutSkeletalMeshReductionSettings.WeldingThreshold = InStaticMeshReductionSettings.WeldingThreshold;
	OutSkeletalMeshReductionSettings.NormalsThreshold = InStaticMeshReductionSettings.HardAngleThreshold;
	OutSkeletalMeshReductionSettings.bRecalcNormals = InStaticMeshReductionSettings.bRecalculateNormals;
	
	OutSkeletalMeshReductionSettings.BaseLOD = InStaticMeshReductionSettings.BaseLODModel;
	
	OutSkeletalMeshReductionSettings.SilhouetteImportance = ConvertOptimizationImportance(InStaticMeshReductionSettings.SilhouetteImportance);
	OutSkeletalMeshReductionSettings.TextureImportance = ConvertOptimizationImportance(InStaticMeshReductionSettings.TextureImportance);
	OutSkeletalMeshReductionSettings.ShadingImportance = ConvertOptimizationImportance(InStaticMeshReductionSettings.ShadingImportance);
	
	switch(InStaticMeshReductionSettings.TerminationCriterion)
	{
	case EStaticMeshReductionTerimationCriterion::Triangles:
		OutSkeletalMeshReductionSettings.TerminationCriterion = SMTC_NumOfTriangles;
		break;
	case EStaticMeshReductionTerimationCriterion::Vertices:
		OutSkeletalMeshReductionSettings.TerminationCriterion = SMTC_NumOfVerts;
		break;
	case EStaticMeshReductionTerimationCriterion::Any:
		OutSkeletalMeshReductionSettings.TerminationCriterion = SMTC_TriangleOrVert;
		break;
	}
}



static bool AddLODFromMeshDescription(
	FMeshDescription&& InMeshDescription,
	USkeletalMesh* InSkeletalMesh,
	IMeshUtilities& InMeshUtilities,
	const bool bCacheOptimize = true)
{
	FSkeletalMeshModel* ImportedModels = InSkeletalMesh->GetImportedModel();
	const int32 LODIndex = ImportedModels->LODModels.Num(); 
	ImportedModels->LODModels.Add(new FSkeletalMeshLODModel);
	if (!ensure(ImportedModels->LODModels.Num() == InSkeletalMesh->GetLODNum()))
	{
		return false;
	}

	FSkeletalMeshImportData SkeletalMeshImportGeometry = FSkeletalMeshImportData::CreateFromMeshDescription(InMeshDescription);

	InSkeletalMesh->CreateMeshDescription(LODIndex, MoveTemp(InMeshDescription));
	InSkeletalMesh->CommitMeshDescription(LODIndex);
	
	FSkeletalMeshLODModel& SkeletalMeshModel = ImportedModels->LODModels.Last();

	// We need at least one set of texture coordinates. Always.
	SkeletalMeshModel.NumTexCoords = FMath::Max<uint32>(1, SkeletalMeshImportGeometry.NumTexCoords);
	
	// Data needed by BuildSkeletalMesh
	TArray<FVector3f> LODPoints;
	TArray<SkeletalMeshImportData::FMeshWedge> LODWedges;
	TArray<SkeletalMeshImportData::FMeshFace> LODFaces;
	TArray<SkeletalMeshImportData::FVertInfluence> LODInfluences;
	TArray<int32> LODPointToRawMap;
	SkeletalMeshImportGeometry.CopyLODImportData( LODPoints, LODWedges, LODFaces, LODInfluences, LODPointToRawMap );

	IMeshUtilities::MeshBuildOptions BuildOptions;
	BuildOptions.TargetPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
	BuildOptions.FillOptions(InSkeletalMesh->GetLODInfo(InSkeletalMesh->GetLODNum() - 1)->BuildSettings);
	BuildOptions.bCacheOptimize = bCacheOptimize;

	TArray<FText> WarningMessages;
	if (!InMeshUtilities.BuildSkeletalMesh(SkeletalMeshModel, InSkeletalMesh->GetPathName(), InSkeletalMesh->GetRefSkeleton(), LODInfluences, LODWedges, LODFaces, LODPoints, LODPointToRawMap, BuildOptions, &WarningMessages, nullptr))
	{
		for(const FText& Message: WarningMessages)
		{
			UE_LOGF(LogStaticToSkeletalMeshConverter, Warning, "%ls", *Message.ToString());
		}
		return false;
	}
	
	return true;
}


static bool AddLODFromStaticMeshSourceModel(
	const FStaticMeshSourceModel& InStaticMeshSourceModel,
	USkeletalMesh* InSkeletalMesh,
	const FBoneIndexType InBoneIndex,
	IMeshUtilities& InMeshUtilities
	)
{
	// Always copy the build and reduction settings. 
	FSkeletalMeshLODInfo& SkeletalLODInfo = InSkeletalMesh->AddLODInfo();

	SkeletalLODInfo.ScreenSize = InStaticMeshSourceModel.ScreenSize;
	CopyBuildSettings(InStaticMeshSourceModel.BuildSettings, SkeletalLODInfo.BuildSettings);
	CopyReductionSettings(InStaticMeshSourceModel.ReductionSettings, SkeletalLODInfo.ReductionSettings);

	FSkeletalMeshModel* ImportedModels = InSkeletalMesh->GetImportedModel();
	const int32 LODIndex = ImportedModels->LODModels.Num(); 

	if (InStaticMeshSourceModel.IsMeshDescriptionValid())
	{
		FMeshDescription SkeletalMeshGeometry;
		if (!InStaticMeshSourceModel.CloneMeshDescription(SkeletalMeshGeometry))
		{
			return false;
		}
		
		FSkeletalMeshAttributes SkeletalMeshAttributes(SkeletalMeshGeometry);
		SkeletalMeshAttributes.Register();
		
		// Fill Bones data.
		const FReferenceSkeleton RefSkeleton = InSkeletalMesh->GetRefSkeleton();
		const int32 NumRefBones = InSkeletalMesh->GetRefSkeleton().GetRawBoneNum();
		
		FSkeletalMeshAttributes::FBoneArray& Bones = SkeletalMeshAttributes.Bones();
		Bones.Reset(NumRefBones);
	
		FSkeletalMeshAttributes::FBoneNameAttributesRef BoneNames = SkeletalMeshAttributes.GetBoneNames();
		FSkeletalMeshAttributes::FBoneParentIndexAttributesRef BoneParentIndices = SkeletalMeshAttributes.GetBoneParentIndices();
		FSkeletalMeshAttributes::FBonePoseAttributesRef BonePoses = SkeletalMeshAttributes.GetBonePoses();
		
		for (int Index = 0; Index < NumRefBones; ++Index)
		{
			const FMeshBoneInfo& BoneInfo = RefSkeleton.GetRawRefBoneInfo()[Index];
			const FTransform& BoneTransform = RefSkeleton.GetRawRefBonePose()[Index];

			const FBoneID BoneID = SkeletalMeshAttributes.CreateBone();

			BoneNames.Set(BoneID, BoneInfo.Name);
			BoneParentIndices.Set(BoneID, BoneInfo.ParentIndex);
			BonePoses.Set(BoneID, BoneTransform);
		}
		
		// Full binding to the root bone.
		FSkinWeightsVertexAttributesRef SkinWeights = SkeletalMeshAttributes.GetVertexSkinWeights();
		UE::AnimationCore::FBoneWeight RootInfluence(InBoneIndex, 1.0f);
		UE::AnimationCore::FBoneWeights RootBinding = UE::AnimationCore::FBoneWeights::Create({RootInfluence});
		
		for (const FVertexID VertexID: SkeletalMeshGeometry.Vertices().GetElementIDs())
		{
			SkinWeights.Set(VertexID, RootBinding);
		}

		// Convert weird static mesh inverse sRGB gamma to linear.
		// FIXME: Remove once static mesh color space has been fixed to be linear again.
		TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = SkeletalMeshAttributes.GetVertexInstanceColors();
		auto ConvertLinearToSRGBGamma = [](float V)
		{
			V = FMath::Clamp(V, 0.0f, 1.0f);
			if (V <= 0.0031308)
			{
				return V * 12.92f;
			}
			else
			{
				return 1.055f * FMath::Pow(V, 1.0f / 2.4f) - 0.055f;
			}
		};
		
		for (FVertexInstanceID VertexInstanceID: SkeletalMeshGeometry.VertexInstances().GetElementIDs())
		{
			FLinearColor VertexColor = VertexInstanceColors.Get(VertexInstanceID);
			VertexColor.R = ConvertLinearToSRGBGamma(VertexColor.R);
			VertexColor.G = ConvertLinearToSRGBGamma(VertexColor.G);
			VertexColor.B = ConvertLinearToSRGBGamma(VertexColor.B);
			VertexInstanceColors.Set(VertexInstanceID, VertexColor);
		}
		

		if (!AddLODFromMeshDescription(MoveTemp(SkeletalMeshGeometry), InSkeletalMesh, InMeshUtilities))
		{
			return false;
		}
	}
	else
	{
		ImportedModels->LODModels.Add(new FSkeletalMeshLODModel);
		
		FSkeletalMeshUpdateContext UpdateContext;
		UpdateContext.SkeletalMesh = InSkeletalMesh;
		
		FLODUtilities::SimplifySkeletalMeshLOD(UpdateContext, LODIndex, GetTargetPlatformManagerRef().GetRunningTargetPlatform());
	}
	
	return true;
}


static bool HasVertexColors(
	const USkeletalMesh* InSkeletalMesh
	)
{
	for (const FSkeletalMeshLODModel& LODModel: InSkeletalMesh->GetImportedModel()->LODModels)
	{
		for (const FSkelMeshSection& Section: LODModel.Sections)
		{
			for (const FSoftSkinVertex& Vertex: Section.SoftVertices)
			{
				if (Vertex.Color != FColor::White)
				{
					return true;
				}
			}
		}
	}
	return false;
}


bool FStaticToSkeletalMeshConverter::InitializeSkeletalMeshFromStaticMesh(
	USkeletalMesh* InSkeletalMesh,
	const UStaticMesh* InStaticMesh,
	const FReferenceSkeleton& InReferenceSkeleton,
	const FName InBindBone 
)
{
	if (!ensure(InSkeletalMesh))
	{
		return false;
	}

	if (!InSkeletalMesh->GetImportedModel()->LODModels.IsEmpty())
	{
		UE_LOGF(LogStaticToSkeletalMeshConverter, Error, "Skeletal mesh '%ls' is not empty", *InSkeletalMesh->GetPathName());
		return false;
	}

	if (!ensure(InStaticMesh))
	{
		return false;
	}

	int32 BoneIndex = 0;
	if (!InBindBone.IsNone())
	{
		BoneIndex = InReferenceSkeleton.FindRawBoneIndex(InBindBone);
		if (BoneIndex == INDEX_NONE)
		{
			UE_LOGF(LogStaticToSkeletalMeshConverter, Error, "Bone '%ls' not found in skeleton.", *InBindBone.ToString());
			return false;
		}
	}

	// This ensures that the render data gets built before we return, by calling PostEditChange when we fall out of scope.
	FScopedSkeletalMeshPostEditChange ScopedPostEditChange( InSkeletalMesh );
	InSkeletalMesh->PreEditChange( nullptr );
	InSkeletalMesh->SetRefSkeleton(InReferenceSkeleton);
	
	// Calculate the initial pose from the reference skeleton.
	InSkeletalMesh->CalculateInvRefMatrices();

	IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>( "MeshUtilities" );
	
	// Copy the LODs and LOD settings over (as close as we can).
	bool bFirstSourceModel = true;
	for (const FStaticMeshSourceModel& StaticMeshSourceModel: InStaticMesh->GetSourceModels())
	{
		if (!AddLODFromStaticMeshSourceModel(
			StaticMeshSourceModel, InSkeletalMesh, static_cast<FBoneIndexType>(BoneIndex), MeshUtilities))
		{
			// If we didn't get a model for LOD index 0, we don't have a mesh. Bail out.
			if (bFirstSourceModel)
			{
				return false;
			}

			// Otherwise, we have a model, so let's continue with what we have.
			break;
		}
		bFirstSourceModel = false;
	}
	
	// Convert the materials over.
	TArray<FSkeletalMaterial> Materials;
	for (const FStaticMaterial& StaticMaterial: InStaticMesh->GetStaticMaterials())
	{
		FSkeletalMaterial Material(
			StaticMaterial.MaterialInterface,
			StaticMaterial.MaterialSlotName,
			StaticMaterial.ImportedMaterialSlotName);
		
		Materials.Add(Material);
	}

	InSkeletalMesh->SetMaterials(Materials);

	if (HasVertexColors(InSkeletalMesh))
	{
		InSkeletalMesh->SetHasVertexColors(true);	
		InSkeletalMesh->SetVertexColorGuid(FGuid::NewGuid());
	}
	
	// Set the bounds from the static mesh, including the extensions, otherwise it won't render properly (among other things).
	InSkeletalMesh->SetImportedBounds( InStaticMesh->GetBounds() );
	InSkeletalMesh->SetPositiveBoundsExtension(InStaticMesh->GetPositiveBoundsExtension());
	InSkeletalMesh->SetNegativeBoundsExtension(InStaticMesh->GetNegativeBoundsExtension());

	//Create some import data so we can re-import this new skeletalmesh
	UAssetImportData* OriginalAssetImportData = InStaticMesh->GetAssetImportData();
	if (OriginalAssetImportData)
	{
		UAssetImportData* DuplicateAssetImportData = DuplicateObject<UAssetImportData>(OriginalAssetImportData, InSkeletalMesh);
		DuplicateAssetImportData->ConvertAssetImportDataToNewOwner(InSkeletalMesh);
		InSkeletalMesh->SetAssetImportData(DuplicateAssetImportData);
	}
	return true;
}


static bool ValidateSkinWeightAttribute(
	const FMeshDescription& InMeshDescription,
	const FReferenceSkeleton& InReferenceSkeleton
	)
{
	using namespace UE::AnimationCore;
	FSkeletalMeshConstAttributes MeshAttributes{InMeshDescription};

	TArray<FName> Profiles = MeshAttributes.GetSkinWeightProfileNames(); 
	if (Profiles.IsEmpty())
	{
		UE_LOGF(LogStaticToSkeletalMeshConverter, Error, "Mesh description doesn't have a skin weight attribute.");
		return false;
	}

	FBoneIndexType BoneIndexMax = static_cast<FBoneIndexType>(InReferenceSkeleton.GetRawBoneNum());
		
	// We use the first profile. Usually that's the default profile, unless we have nothing but alternate profiles.
	FSkinWeightsVertexAttributesConstRef VertexSkinWeights = MeshAttributes.GetVertexSkinWeights(Profiles[0]);
	for (const FVertexID VertexID: InMeshDescription.Vertices().GetElementIDs())
	{
		for (FBoneWeight BoneWeight: VertexSkinWeights.Get(VertexID))
		{
			if (BoneWeight.GetBoneIndex() >= BoneIndexMax)
			{
				UE_LOGF(LogStaticToSkeletalMeshConverter, Error, "Mesh description's skin weight refers to a non-existent bone (%d of %d).", BoneWeight.GetBoneIndex(), BoneIndexMax);
				return false;
			}
		}
	}
	return true;
}


bool FStaticToSkeletalMeshConverter::InitializeSkeletalMeshFromMeshDescriptions(
	USkeletalMesh* InSkeletalMesh,
	TArrayView<const FMeshDescription*> InMeshDescriptions,
	TConstArrayView<FSkeletalMaterial> InMaterials,
	const FReferenceSkeleton& InReferenceSkeleton,
	const bool bInRecomputeNormals,
	const bool bInRecomputeTangents,
	const bool bCacheOptimize
)	
{
	FInitializationParams Parameters;
	Parameters.Materials = InMaterials;
	Parameters.bRecomputeNormals = bInRecomputeNormals;
	Parameters.bRecomputeTangents = bInRecomputeTangents;
	Parameters.bCacheOptimize = bCacheOptimize;
	
	return InitializeSkeletalMeshFromMeshDescriptions(InSkeletalMesh, InMeshDescriptions, InReferenceSkeleton, Parameters);
}

bool FStaticToSkeletalMeshConverter::InitializeSkeletalMeshFromMeshDescriptions(
	USkeletalMesh* InSkeletalMesh,
	TArrayView<const FMeshDescription*> InMeshDescriptions,
	const FReferenceSkeleton& InReferenceSkeleton,
	const FInitializationParams& InParams
)	
{
	if (!ensure(InSkeletalMesh))
	{
		return false;
	}

	if (!InSkeletalMesh->GetImportedModel()->LODModels.IsEmpty())
	{
		UE_LOGF(LogStaticToSkeletalMeshConverter, Error, "Skeletal mesh '%ls' is not empty", *InSkeletalMesh->GetPathName());
		return false;
	}

	if (InMeshDescriptions.IsEmpty())
	{
		UE_LOGF(LogStaticToSkeletalMeshConverter, Error, "No mesh descriptions given");
		return false;
	}

	// Ensure all mesh descriptions have a skin weight attribute.
	for (const FMeshDescription* MeshDescription: InMeshDescriptions)
	{
		if (!ValidateSkinWeightAttribute(*MeshDescription, InReferenceSkeleton))
		{
			return false;
		}
	}

	// Set the materials before we start converting. We'll add dummy materials afterward if there are more sections
	// than materials in any of the LODs. Not the best system, but the best we have for now.
	InSkeletalMesh->SetMaterials(TArray<FSkeletalMaterial>{InParams.Materials});

	TSet<FName> ValidMaterialSlotNames;
	for (int32 Index = 0; Index < InParams.Materials.Num(); Index++)
	{
		const FSkeletalMaterial& Material = InParams.Materials[Index];
		if (!Material.MaterialSlotName.IsNone())
		{
			ValidMaterialSlotNames.Add(Material.MaterialSlotName);
		}
	}
	
	// This ensures that the render data gets built before we return, by calling PostEditChange when we fall out of scope.
	{
		FScopedSkeletalMeshPostEditChange ScopedPostEditChange( InSkeletalMesh );
		InSkeletalMesh->PreEditChange( nullptr );
		InSkeletalMesh->SetRefSkeleton(InReferenceSkeleton);
		
		// Calculate the initial pose from the reference skeleton.
		InSkeletalMesh->CalculateInvRefMatrices();

		IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>( "MeshUtilities" );
		bool bFirstSourceModel = true;
		
		for (const FMeshDescription* MeshDescription: InMeshDescriptions)
		{
			// Add default LOD build settings.
			FSkeletalMeshLODInfo& SkeletalLODInfo = InSkeletalMesh->AddLODInfo();
			SkeletalLODInfo.ReductionSettings.NumOfTrianglesPercentage = 1.0f;
			SkeletalLODInfo.ReductionSettings.NumOfVertPercentage = 1.0f;
			SkeletalLODInfo.ReductionSettings.MaxDeviationPercentage = 0.0f;
			SkeletalLODInfo.LODHysteresis = 0.02f;

			SkeletalLODInfo.BuildSettings.bRecomputeNormals = InParams.bRecomputeNormals;
			SkeletalLODInfo.BuildSettings.bRecomputeTangents = InParams.bRecomputeTangents;

			FMeshDescription ClonedDescription(*MeshDescription);

			// Fix up the material slot names on the mesh to match the ones in the material list. If the name is
			// either NAME_None, or doesn't exist in the material list, we use the group index to index into the
			// material list to resolve the name.
			FSkeletalMeshAttributes Attributes(ClonedDescription);
			TPolygonGroupAttributesRef<FName> MaterialSlotNamesAttribute = Attributes.GetPolygonGroupMaterialSlotNames();
			for (FPolygonGroupID PolygonGroupID: ClonedDescription.PolygonGroups().GetElementIDs())
			{
				if (!ValidMaterialSlotNames.Contains(MaterialSlotNamesAttribute.Get(PolygonGroupID)))
				{
					int32 MaterialIndex = PolygonGroupID.GetValue();
					MaterialIndex = FMath::Clamp(MaterialIndex, 0, InParams.Materials.Num() - 1);
					if (InParams.Materials.IsValidIndex(MaterialIndex))
					{
						MaterialSlotNamesAttribute.Set(PolygonGroupID, InParams.Materials[MaterialIndex].MaterialSlotName);
					}
				}
			}
			
			// pre-add the vertex attributes to enable for render so that the render mesh generation takes them into account during creation. 
			if (!InParams.VertexAttributesForRender.IsEmpty())
			{
				const TAttributesSet<FVertexID>& VertexAttributes = MeshDescription->VertexAttributes();
				VertexAttributes.ForEachByType<float>([&InParams, &SkeletalLODInfo](const FName AttributeName, TVertexAttributesConstRef<float>)
				{
					if (!FSkeletalMeshAttributes::IsReservedAttributeName(AttributeName))
					{
						if (InParams.VertexAttributesForRender.Contains(AttributeName))
						{
							FSkeletalMeshVertexAttributeInfo& NewAttribute = SkeletalLODInfo.VertexAttributes.AddDefaulted_GetRef();
							NewAttribute.Name = AttributeName;
							NewAttribute.EnabledForRender = true;
						}
					}
				});
			}
			
			if (!AddLODFromMeshDescription(MoveTemp(ClonedDescription), InSkeletalMesh, MeshUtilities, InParams.bCacheOptimize))
			{
				// If we didn't get a model for LOD index 0, we don't have a mesh. Bail out.
				if (bFirstSourceModel)
				{
					return false;
				}

				// Otherwise, we have a model, so let's continue with what we have.
				break;
			}
			bFirstSourceModel = false;
		}
	}

	// Compute the bbox, now that we have the model mesh generated. 
	FBox3f BoundingBox{ForceInit};
	int32 MaxSectionCount = 0;
	for (const FSkeletalMeshLODModel& MeshModel: InSkeletalMesh->GetImportedModel()->LODModels)
	{
		MaxSectionCount = FMath::Max(MaxSectionCount, MeshModel.Sections.Num());

		// Compute the overall bbox.
		for (const FSkelMeshSection& Section: MeshModel.Sections)
		{
			for (const FSoftSkinVertex& Vertex: Section.SoftVertices)
			{
				BoundingBox += Vertex.Position;
			}
		}
	}

	// If we're short on materials, compared to sections, add dummy materials to fill in the gap. Not ideal, but
	// best we can do for now.
	const TArray<FSkeletalMaterial>& ExistingMaterials = InSkeletalMesh->GetMaterials();
	if (MaxSectionCount > ExistingMaterials.Num())
	{
		TArray<FSkeletalMaterial> NewMaterials{ExistingMaterials};
		for (int32 Index = ExistingMaterials.Num(); Index < MaxSectionCount; Index++)
		{
			NewMaterials.Add(FSkeletalMaterial{});
		}
		InSkeletalMesh->SetMaterials(NewMaterials);
	}

	InSkeletalMesh->SetImportedBounds( FBox3d{BoundingBox} );

	return true;
}

#endif
