// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"
#include "MeshBuild.h"
#include "MeshUtilities.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Framework/Commands/UIAction.h"
#include "Animation/MorphTarget.h"


namespace ClothingAssetUtils
{
	struct FClothingAssetMeshBinding;
}

//////////////////////////////////////////////////////////////////////////
// FSkeletalMeshUpdateContext


struct FSkeletalMeshUpdateContext
{
	USkeletalMesh*				SkeletalMesh;
	TArray<UActorComponent*>	AssociatedComponents;

	FExecuteAction				OnLODChanged;
};

/* Helper struct to define inline data when applying morph target on a reduce LOD. Inline data is needed when reduction is inline (LOD reduce itself). */
struct FInlineReductionDataParameter
{
	bool bIsDataValid = false;
	FSkeletalMeshLODModel InlineOriginalSrcModel;
	TMap<FString, TArray<FMorphTargetDelta>> InlineOriginalSrcMorphTargetData;
};

class FSkeletalMeshImportData;

//////////////////////////////////////////////////////////////////////////
// FLODUtilities


class FLODUtilities
{
public:

	/**
	* Process and update the vertex Influences using the predefined wedges
	* 
	* @param VertexCount - The number of wedges in the corresponding mesh.
	* @param Influences - BoneWeights and Ids for the corresponding vertices. 
	* @param MeshName	- Name of mesh, used for warning messages
	*/
	static SKELETALMESHUTILITIESCOMMON_API void ProcessImportMeshInfluences(const int32 VertexCount, TArray<SkeletalMeshImportData::FRawBoneInfluence>& Influences, const FString& MeshName);

	/** Regenerate LODs of the mesh
	*
	* @param SkeletalMesh : the mesh that will regenerate LOD
	* @param NewLODCount : Set valid value (>0) if you want to change LOD count.
	*						Otherwise, it will use the current LOD and regenerate
	* @param bRegenerateEvenIfImported : If this is true, it only regenerate even if this LOD was imported before
	*									If false, it will regenerate for only previously auto generated ones
	*
	* @return true if succeed. If mesh reduction is not available this will return false.
	*/
	static SKELETALMESHUTILITIESCOMMON_API bool RegenerateLOD(USkeletalMesh* SkeletalMesh, const class ITargetPlatform* TargetPlatform, int32 NewLODCount = 0, bool bRegenerateEvenIfImported = false, bool bGenerateBaseLOD = false);

	/** Removes a particular LOD from the SkeletalMesh. 
	*
	* @param UpdateContext - The skeletal mesh and actor components to operate on.
	* @param DesiredLOD   - The LOD index to remove the LOD from.
	*/
	static SKELETALMESHUTILITIESCOMMON_API void RemoveLOD( FSkeletalMeshUpdateContext& UpdateContext, int32 DesiredLOD );

	/** Removes the specified LODs from the SkeletalMesh.
	*
	* @param UpdateContext - The skeletal mesh and actor components to operate on.
	* @param DesiredLODs   - The array of LOD index to remove the LOD from. The order is irrelevant since the array will be sorted to be reverse iterate.
	*/
	static SKELETALMESHUTILITIESCOMMON_API void RemoveLODs(FSkeletalMeshUpdateContext& UpdateContext, const TArray<int32>& DesiredLODs);

	/*
	 * Add or change the LOD data specified by LodIndex with the content of the sourceSkeletalMesh.
	 *
	 * @Param DestinationSkeletalMesh - Skeletal mesh receiving the LOD data.
	 * @Param SourceSkeletalMesh - Skeletal mesh providing the LOD data we want to use to add or modify the specified destination LOD at LodIndex. The LOD we want to import is always the base LOD of the source mesh.
	 * @Param LodIndex - The destination lod index we want to add or modify.
	 * @Param SourceDataFilename - The file name we use to import the source skeletal mesh.
	 */
	static SKELETALMESHUTILITIESCOMMON_API bool SetCustomLOD(USkeletalMesh* DestinationSkeletalMesh, USkeletalMesh* SourceSkeletalMesh, const int32 LodIndex, const FString& SourceDataFilename);

	/**
	*	Simplifies the static mesh based upon various user settings for DesiredLOD.
	*
	* @param UpdateContext - The skeletal mesh and actor components to operate on.
	* @param DesiredLOD - The LOD to simplify
	*/
	static SKELETALMESHUTILITIESCOMMON_API void SimplifySkeletalMeshLOD(FSkeletalMeshUpdateContext& UpdateContext, int32 DesiredLOD, const class ITargetPlatform* TargetPlatform, bool bRestoreClothing = false, std::atomic<bool>* OutNeedsPackageDirtied = nullptr);

	/**
	*	Restore the LOD imported model to the last imported data. Call this function if you want to remove the reduce on the base LOD
	*
	* @param SkeletalMesh - The skeletal mesh to operate on.
	* @param LodIndex - The LOD index to restore the imported LOD model
	* @param bReregisterComponent - if true the component using the skeletal mesh will all be re register.
	*/
	static SKELETALMESHUTILITIESCOMMON_API bool RestoreSkeletalMeshLODImportedData_DEPRECATED(USkeletalMesh* SkeletalMesh, int32 LodIndex);
	
	/**
	 * Refresh LOD Change
	 * 
	 * LOD has changed, it will have to notify all SMC that uses this SM
	 * and ask them to refresh LOD
	 *
	 * @param	InSkeletalMesh	SkeletalMesh that LOD has been changed for
	 */
	static SKELETALMESHUTILITIESCOMMON_API void RefreshLODChange(const USkeletalMesh* SkeletalMesh);


	/*
	 * Regenerate LOD that are dependent of LODIndex
	 */
	static SKELETALMESHUTILITIESCOMMON_API void RegenerateDependentLODs(USkeletalMesh* SkeletalMesh, int32 LODIndex, const class ITargetPlatform* TargetPlatform);

	/*
	 * Build the morph targets for the specified LOD. The function use the Morph target data stored in the FSkeletalMeshImportData ImportData structure
	 */
	static SKELETALMESHUTILITIESCOMMON_API void BuildMorphTargets(USkeletalMesh* SkeletalMesh, class FSkeletalMeshImportData &ImportData, int32 LODIndex, bool ShouldImportNormals, bool ShouldImportTangents, bool bUseMikkTSpace, const FOverlappingThresholds& Thresholds);

	/*
	 * Same as above but use normals from the source mesh description to build up the morph targets. 
	 */
	static SKELETALMESHUTILITIESCOMMON_API void BuildMorphTargets(
		USkeletalMesh* SkeletalMesh,
		const FMeshDescription& SkeletalMeshModel,
		FSkeletalMeshImportData& ImportData,
		int32 LODIndex,
		bool ShouldImportNormals,
		bool ShouldImportTangents,
		bool bUseMikkTSpace,
		const FOverlappingThresholds& Thresholds
		);
	
	
	/**
	 *	This function apply the skinning weights from asource skeletal mesh to the destination skeletal mesh.
	 *  The Destination will receive the weights has the alternate weights.
	 *  We extract the imported skinning weight data from the SkeletalMeshSrc and we save the imported raw data into the destination mesh.
	 *  Then we call UpdateAlternateSkinWeights without the SkeletalMeshSrc
	 *
	 * @param SkeletalMeshDest - The skeletal mesh that will receive the alternate skinning weights.
	 * @param SkeletalMeshSrc - The skeletal mesh that contain the alternate skinning weights.
	 * @param LODIndexDest - the destination LOD
	 * @param LODIndexSrc - the Source LOD index
	 */
	static SKELETALMESHUTILITIESCOMMON_API bool UpdateAlternateSkinWeights(
		USkeletalMesh* SkeletalMeshDest,
		const FName& ProfileNameDest,
		USkeletalMesh* SkeletalMeshSrc,
		int32 LODIndexDest,
		int32 LODIndexSrc,
		const IMeshUtilities::MeshBuildOptions& Options);

	UE_DEPRECATED(5.2, "Please use the new overloads of UpdateAlternateSkinWeights that take an IMeshUtilities::MeshBuildOptions. Note that IMeshUtilities::MeshBuildOptions::bComputeNormals/Tangents has the opposite meaning of ShouldImportNormals/Tangents.")
	static SKELETALMESHUTILITIESCOMMON_API bool UpdateAlternateSkinWeights(
		USkeletalMesh* SkeletalMeshDest,
		const FName& ProfileNameDest,
		USkeletalMesh* SkeletalMeshSrc,
		int32 LODIndexDest,
		int32 LODIndexSrc,
		FOverlappingThresholds OverlappingThresholds,
		bool ShouldImportNormals,
		bool ShouldImportTangents,
		bool bUseMikkTSpace,
		bool bComputeWeightedNormals);

	/*
	 *	This function apply the skinning weights from the saved imported skinning weight data to the destination skeletal mesh.
	 *  The Destination will receive the weights has the alternate weights.
	 *
	 * @param SkeletalMeshDest - The skeletal mesh that will receive the alternate skinning weights.
	 * @param LODIndexDest - the destination LOD
	 */
	static SKELETALMESHUTILITIESCOMMON_API bool UpdateAlternateSkinWeights(
		USkeletalMesh* SkeletalMeshDest,
		const FName& ProfileNameDest,
		int32 LODIndexDest,
		const IMeshUtilities::MeshBuildOptions& Options);

	UE_DEPRECATED(5.2, "Please use the new overloads of UpdateAlternateSkinWeights that take an IMeshUtilities::MeshBuildOptions. Note that IMeshUtilities::MeshBuildOptions::bComputeNormals/Tangents has the opposite meaning of ShouldImportNormals/Tangents.")
	static SKELETALMESHUTILITIESCOMMON_API bool UpdateAlternateSkinWeights(
		USkeletalMesh* SkeletalMeshDest,
		const FName& ProfileNameDest,
		int32 LODIndexDest,
		FOverlappingThresholds OverlappingThresholds,
		bool ShouldImportNormals,
		bool ShouldImportTangents,
		bool bUseMikkTSpace,
		bool bComputeWeightedNormals);

	static SKELETALMESHUTILITIESCOMMON_API bool UpdateAlternateSkinWeights(
		FSkeletalMeshLODModel& LODModelDest,
		FSkeletalMeshImportData& ImportDataDest,
		USkeletalMesh* SkeletalMeshDest,
		const FReferenceSkeleton& RefSkeleton,
		const FName& ProfileNameDest,
		int32 LODIndexDest,
		const IMeshUtilities::MeshBuildOptions& Options);

	UE_DEPRECATED(5.2, "Please use the new overloads of UpdateAlternateSkinWeights that take an IMeshUtilities::MeshBuildOptions. Note that IMeshUtilities::MeshBuildOptions::bComputeNormals/Tangents has the opposite meaning of ShouldImportNormals/Tangents.")
	static SKELETALMESHUTILITIESCOMMON_API bool UpdateAlternateSkinWeights(
		FSkeletalMeshLODModel& LODModelDest,
		FSkeletalMeshImportData& ImportDataDest,
		USkeletalMesh* SkeletalMeshDest,
		const FReferenceSkeleton& RefSkeleton,
		const FName& ProfileNameDest,
		int32 LODIndexDest,
		FOverlappingThresholds OverlappingThresholds,
		bool ShouldImportNormals,
		bool ShouldImportTangents,
		bool bUseMikkTSpace,
		bool bComputeWeightedNormals);

	
	/** Build the vertex attributes */
	static SKELETALMESHUTILITIESCOMMON_API bool UpdateLODInfoVertexAttributes(USkeletalMesh *InSkeletalMesh, int32 InSourceLODIndex, int32 InTargetLODIndex, bool bInCopyAttributeValues);
	
	/**
	 * Re-generate all (editor-only) skin weight profile, used whenever we rebuild the skeletal mesh data which could change the chunking and bone indices
	 * 
	 * If BoneInfluenceLimit is 0, the DefaultBoneInfluenceLimit from the project settings will be used.
	 */
	static SKELETALMESHUTILITIESCOMMON_API void RegenerateAllImportSkinWeightProfileData(FSkeletalMeshLODModel& LODModelDest, int32 BoneInfluenceLimit = 0, const ITargetPlatform* TargetPlatform = nullptr);

	static SKELETALMESHUTILITIESCOMMON_API void UnbindClothingAndBackup(USkeletalMesh* SkeletalMesh, TArray<ClothingAssetUtils::FClothingAssetMeshBinding>& ClothingBindings);
	static SKELETALMESHUTILITIESCOMMON_API void UnbindClothingAndBackup(USkeletalMesh* SkeletalMesh, TArray<ClothingAssetUtils::FClothingAssetMeshBinding>& ClothingBindings, const int32 LODIndex);
	
	static SKELETALMESHUTILITIESCOMMON_API void RestoreClothingFromBackup(USkeletalMesh* SkeletalMesh, TArray<ClothingAssetUtils::FClothingAssetMeshBinding>& ClothingBindings);
	static SKELETALMESHUTILITIESCOMMON_API void RestoreClothingFromBackup(USkeletalMesh* SkeletalMesh, TArray<ClothingAssetUtils::FClothingAssetMeshBinding>& ClothingBindings, const int32 LODIndex);

	struct FMorphTargetLodBackupData
	{
		bool bIsEmpty = true;
		TArray<FVector3f> MorphPositionDeltas;
		TArray<FVector3f> MorphNormalDeltas;
	};

	/**
	 * Backup all the custom imported morph targets.
	 */
	static SKELETALMESHUTILITIESCOMMON_API void BackupCustomImportedMorphTargetData(USkeletalMesh* SkeletalMesh, TMap<FString, TArray<FMorphTargetLodBackupData>>& BackupImportedMorphTargetData);

	/**
	 * Restore the custom imported morph target in the mesh description. Return true if the mesh description was changed because it restore some morph targets
	 */
	static SKELETALMESHUTILITIESCOMMON_API bool RestoreCustomImportedMorphTargetData(USkeletalMesh* SkeletalMesh, const int32 LodIndex, FMeshDescription& LodMeshDescription, const TMap<FString, TArray<FMorphTargetLodBackupData>>& BackupImportedMorphTargetData);

	/**
	 * Before building skeletalmesh base LOD (LOD index 0) using MeshUtilities.BuildSkeletalMesh, we want to adjust the imported faces material index to point on the correct sk material. We use the material name to match the material.
	 * @param Materials - The skeletalmesh material list to fit the import data face material
	 * @param RawMeshMaterials - The import data material list
	 * @param LODFaces - The face to adjust with the material remap
	 * @param LODIndex - We can adjust only the base LOD (LODIndex 0), the function will not do anything if LODIndex is not 0
	 */
	static SKELETALMESHUTILITIESCOMMON_API void AdjustImportDataFaceMaterialIndex(const TArray<FSkeletalMaterial>& Materials, TArray<SkeletalMeshImportData::FMaterial>& RawMeshMaterials, TArray<SkeletalMeshImportData::FMeshFace>& LODFaces, int32 LODIndex);

	/**
	 * Structure to pass all the needed parameters to do match the material when importing a skeletal mesh LOD.
	 * @param SkeletalMesh - The skeletalmesh that get a LOD re/import
	 * @param LodIndex - The skeletalmesh LOD index that get re/import
	 * @param bIsReImport - whether its a skeletal mesh LOD re-import
	 * @param ImportedMaterials - The imported material list
	 * @param ExistingOriginalPerSectionMaterialImportName - The previously LOD imported material list (cannot be null if bIsReImport is true)
	 * @param CustomImportedLODModel - When importing custom LOD we want to pass the LODModel to synchronize the sections
	 */
	struct FSkeletalMeshMatchImportedMaterialsParameters
	{
		USkeletalMesh* SkeletalMesh = nullptr;
		int32 LodIndex = 0;
		bool bIsReImport = false;
		const TArray<SkeletalMeshImportData::FMaterial>* ImportedMaterials = nullptr;
		const TArray<FName>* ExistingOriginalPerSectionMaterialImportName = nullptr;
		FSkeletalMeshLODModel* CustomImportedLODModel = nullptr;
	};

	/**
	 * When any skeletalmesh LOD get imported or re-imported, we want to have common code to set all materials data.
	 * In case we re-import a LOD this code will match the sections using imported material name to be able to keep any section data.
	 * The material slot array is not re-order even if we re-import the base LOD.
	 */
	static SKELETALMESHUTILITIESCOMMON_API void MatchImportedMaterials(FSkeletalMeshMatchImportedMaterialsParameters& Parameters);

	/**
	 * Reorder the material slot array to follow the base LOD section order. It will readjust all LOD section material index and LODMaterialMap.
	 */
	static SKELETALMESHUTILITIESCOMMON_API void ReorderMaterialSlotToBaseLod(USkeletalMesh* SkeletalMesh);

	/**
	 * Remove any material slot that is not used by any LODs
	 */
	static SKELETALMESHUTILITIESCOMMON_API void RemoveUnusedMaterialSlot(USkeletalMesh* SkeletalMesh);

	/**
	 * This function will strip all triangle in the specified LOD that don't have any UV area pointing on a black pixel in the TextureMask.
	 * We use the UVChannel 0 to find the pixels in the texture.
	 *
	 * @Param SkeletalMesh: The skeletalmesh we want to optimize
	 * @Param LODIndex: The LOD we want to optimize
	 * @Param TextureMask: The texture containing the stripping mask. non black pixel strip triangle, black pixel keep them.
	 * @Param Threshold: The threshold we want when comparing the texture value with zero.
	 */
	static SKELETALMESHUTILITIESCOMMON_API bool StripLODGeometry(USkeletalMesh* SkeletalMesh, const int32 LODIndex, UTexture2D* TextureMask, const float Threshold);

private:
	FLODUtilities() {}

	/** Generate the editor-only data stored for a skin weight profile (relies on bone indices) */
	static SKELETALMESHUTILITIESCOMMON_API void GenerateImportedSkinWeightProfileData(
		FSkeletalMeshLODModel& LODModelDest,
		struct FImportedSkinWeightProfileData& ImportedProfileData,
		int32 BoneInfluenceLimit,
		const class ITargetPlatform* TargetPlatform);

	struct FMorphTargetUpdateRequest
	{
		FMorphTargetUpdateRequest() = default;
		FMorphTargetUpdateRequest(USkeletalMesh* InSkeletalMesh, const int32 InSourceLOD, const int32 InDestinationLOD, FInlineReductionDataParameter&& InReductionData) :
			SkeletalMesh(InSkeletalMesh),
			SourceLOD(InSourceLOD),
			DestinationLOD(InDestinationLOD),
			ReductionData(MoveTemp(InReductionData))
		{}

		FMorphTargetUpdateRequest(USkeletalMesh* InSkeletalMesh, const int32 InDestinationLOD) :
			SkeletalMesh(InSkeletalMesh),
			DestinationLOD(InDestinationLOD)
		{}
		
		USkeletalMesh* SkeletalMesh = nullptr; 
		int32 SourceLOD = INDEX_NONE;
		int32 DestinationLOD = INDEX_NONE;
		FInlineReductionDataParameter ReductionData;

		void Run() const
		{
			if (SourceLOD != INDEX_NONE)
			{
				ApplyMorphTargetsToLOD(SkeletalMesh, SourceLOD, DestinationLOD, ReductionData);
			}
			else
			{
				ClearGeneratedMorphTarget(SkeletalMesh, DestinationLOD);
			}
		}
	};

	/**
	 *	Simplifies the static mesh based upon various user settings for DesiredLOD
	 *  This is private function that gets called by SimplifySkeletalMesh
	 *
	 * @param SkeletalMesh - The skeletal mesh and actor components to operate on.
	 * @param DesiredLOD - Desired LOD
	 */
	static SKELETALMESHUTILITIESCOMMON_API void SimplifySkeletalMeshLOD(
		USkeletalMesh* SkeletalMesh, 
		int32 DesiredLOD, 
		const ITargetPlatform* TargetPlatform,
		bool bRestoreClothing, 
		TArray<FMorphTargetUpdateRequest>& OutMorphRequests,
		std::atomic<bool>* OutNeedsPackageDirtied = nullptr
		);

	/**
	*  Remap the morph targets of the base LOD onto the desired LOD.
	*
	* @param SkeletalMesh - The skeletal mesh to operate on.
	* @param SourceLOD      - The source LOD morph target .
	* @param DestinationLOD   - The destination LOD morph target to apply the source LOD morph target
	*/
	static SKELETALMESHUTILITIESCOMMON_API void ApplyMorphTargetsToLOD(USkeletalMesh* SkeletalMesh, int32 SourceLOD, int32 DestinationLOD, const FInlineReductionDataParameter& InlineApplyMorphTargetParameter);

	/**
	*  Clear generated morphtargets for the given LODs
	*
	* @param SkeletalMesh - The skeletal mesh and actor components to operate on.
	* @param DesiredLOD - Desired LOD
	*/
	static SKELETALMESHUTILITIESCOMMON_API void ClearGeneratedMorphTarget(USkeletalMesh* SkeletalMesh, int32 DesiredLOD);
};

#endif //WITH_EDITOR
