// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"
#include "Delegates/Delegate.h"
#include "Containers/Array.h"

#define UE_API SKELETALMESHUTILITIESCOMMON_API

class FName;
class HHitProxy;

enum class ESkeletalMeshNotifyType
{
	BonesAdded,
	BonesRemoved,
	BonesMoved,
	BonesSelected,
	BonesRenamed,
	HierarchyChanged
};

// A delegate for monitoring to skeletal mesh global notifications.
DECLARE_MULTICAST_DELEGATE_TwoParams(FSkeletalMeshNotifyDelegate, const TArray<FName>& /*InBoneNames*/, const ESkeletalMeshNotifyType /*InNotifType*/);

class ISkeletalMeshNotifier
{
public:
	ISkeletalMeshNotifier() = default;
	virtual ~ISkeletalMeshNotifier() = default;

	UE_API FSkeletalMeshNotifyDelegate& Delegate();

	// override this function to react to notifications locally.
	virtual void HandleNotification(const TArray<FName>& BoneNames, const ESkeletalMeshNotifyType InNotifyType) = 0;

	UE_API void Notify(const TArray<FName>& BoneNames, const ESkeletalMeshNotifyType InNotifyType) const;
	UE_API bool Notifying() const;

private:
	FSkeletalMeshNotifyDelegate NotifyDelegate;
	mutable bool bNotifying = false;
};

class FSkeletalMeshNotifierBindScope 
{
	
public:
	UE_API FSkeletalMeshNotifierBindScope(TWeakPtr<ISkeletalMeshNotifier> InNotifierA,  TWeakPtr<ISkeletalMeshNotifier> InNotifierB);
	UE_API ~FSkeletalMeshNotifierBindScope();

private:
	TWeakPtr<ISkeletalMeshNotifier> NotifierA;
	TWeakPtr<ISkeletalMeshNotifier> NotifierB;

	FDelegateHandle	DelegateToRemoveFromNotifierA;	
	FDelegateHandle	DelegateToRemoveFromNotifierB;	
};

class ISkeletalMeshEditorBinding
{
public:
	ISkeletalMeshEditorBinding() = default;
	virtual ~ISkeletalMeshEditorBinding() = default;

	virtual TSharedPtr<ISkeletalMeshNotifier> GetNotifier() = 0;

	using NameFunction = TFunction< TOptional<FName>(HHitProxy*)>;
	virtual NameFunction GetNameFunction() = 0;

	virtual TArray<FName> GetSelectedBones() const = 0;
};

#undef UE_API
