// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Math/MathFwd.h"
#include "UObject/NameTypes.h"


class UObject;
class USkeletalMesh;
class USkeleton;
class UStaticMesh;
struct FMeshDescription;
struct FReferenceSkeleton;
struct FSkeletalMaterial;


struct FStaticToSkeletalMeshConverter
{
	/** Creates a skeleton from a static mesh with a single root bone.
	 *
	 *  @param InSkeleton The skeleton to initialize. It has to be freshly created and contain no bones already.
	 *  @param InStaticMesh The static mesh whose bounding box will be used as a reference.
	 *  @param InRelativeRootPosition The relative root position in a unit cube that gets scaled up to match the
	 *    bbox of the static mesh. For example, given FVector(0.5, 0.5, 0.0), the root bone location will be placed
	 *    at the center of the bottom of the static mesh's bbox.
	 *  @return \c true if the operation succeeded. 
	 */
	static SKELETALMESHUTILITIESCOMMON_API bool InitializeSkeletonFromStaticMesh(
		USkeleton* InSkeleton,
		const UStaticMesh* InStaticMesh,
		const FVector& InRelativeRootPosition 
		);

	/** Creates a skeleton from a static mesh with a bone chain going from root to end effector, where the intermediary
	 *  bones are distributed evenly along a line between the two.
	 *
	 *  @param InSkeleton The skeleton to initialize. It has to be freshly created and contain no bones already.
	 *  @param InStaticMesh The static mesh whose bounding box will be used as a reference.
	 *  @param InRelativeRootPosition The relative root position in a unit cube that gets scaled up to match the
	 *    bbox of the static mesh. For example, given FVector(0.5, 0.5, 0.0), the root bone location will be placed
	 *    at the center of the bottom of the static mesh's bbox.
	 *  @param InRelativeEndEffectorPosition The end effector position, positioned in the same manner as the root
	 *    position. If the end effector is in the same location as the root, only the root bone is created.
	 *  @param InIntermediaryJointCount Number of joints to create between the root and the end effector.  
	 *  @return \c true if the operation succeeded. 
	 */
	static SKELETALMESHUTILITIESCOMMON_API bool InitializeSkeletonFromStaticMesh(
		USkeleton* InSkeleton,
		const UStaticMesh* InStaticMesh,
		const FVector& InRelativeRootPosition,
		const FVector& InRelativeEndEffectorPosition,
		const int32 InIntermediaryJointCount
		);

	/** Initialize a skeletal mesh from the given static mesh and a skeleton. The mesh will initially be created with a
	 *  rigid binding on the root bone. 
	 *  @param InSkeletalMesh The skeletal mesh to initialize. It has to be freshly created and be completely empty.
	 *  @param InStaticMesh The static mesh to convert from.
	 *  @param InReferenceSkeleton The reference skeleton to use.
	 *  @param InBindBone The bone to bind to. If no bone name is given, the binding defaults to the root bone.
	 */
	static SKELETALMESHUTILITIESCOMMON_API bool InitializeSkeletalMeshFromStaticMesh(
		USkeletalMesh* InSkeletalMesh, 
		const UStaticMesh* InStaticMesh,
		const FReferenceSkeleton& InReferenceSkeleton,
		const FName InBindBone = NAME_None 
		);

	/** Create a skeletal mesh from a list of mesh description objects and a reference skeleton. If any mesh description
	 *  either doesn't contain a skin weight, or any of the skin weights reference a bone that doesn't exist on the
	 *  reference skeleton, this operation fails.
	 *  @param InSkeletalMesh The skeletal mesh to initialize. It has to be freshly created and be completely empty.
	 *  @param InMeshDescriptions The mesh description objects to create each subsequent LOD, starting from LOD0.
	 *  @param InMaterials The materials to apply to the skeletal mesh. The conversion will attempt a best effort to
	 *     ensure that the PolygonGroup::ImportedMaterialSlotName attribute will map to existing import names. 
	 *  @param InReferenceSkeleton The reference skeleton to use.
	 *  @param bInRecomputeNormals Recompute normals when the mesh is generated from the mesh description.
	 *  @param bInRecomputeTangents Recompute tangents when the mesh is generated from the mesh description.
	 *  @param bCacheOptimize Enable the cache optimization of the index buffer.
	 */
	UE_DEPRECATED(5.8, "Please use the new overloads of InitializeSkeletalMeshFromMeshDescriptions that takes FInitializationParams.")
	static SKELETALMESHUTILITIESCOMMON_API bool InitializeSkeletalMeshFromMeshDescriptions(
		USkeletalMesh* InSkeletalMesh,
		TArrayView<const FMeshDescription*> InMeshDescriptions,
		TConstArrayView<FSkeletalMaterial> InMaterials,
		const FReferenceSkeleton& InReferenceSkeleton,
		const bool bInRecomputeNormals = false,
		const bool bInRecomputeTangents = true,
		const bool bCacheOptimize = true
		);
	
	/* Structure that defines parameters passed into the initialize skeletal mesh from mesh descriptions function. */
	struct FInitializationParams
	{
		FInitializationParams() {}

		/** The materials to apply to the skeletal mesh. The conversion will attempt a best effort to
	     * ensure that the PolygonGroup::ImportedMaterialSlotName attribute will map to existing import names.
		 */
		TConstArrayView<FSkeletalMaterial> Materials;
		
		/** Recompute normals when the mesh is generated from the mesh description. */
		bool bRecomputeNormals = false;
		
		/** Recompute tangents when the mesh is generated from the mesh description. */
		bool bRecomputeTangents = true;
		
		/** Enable the cache optimization of the index buffer. */
		bool bCacheOptimize = true;
		
		/** Vertex attribute names, as defined in the mesh description, that need to be enabled for render. */
		TConstArrayView<FName> VertexAttributesForRender;
	};
	
	/** Create a skeletal mesh from a list of mesh description objects and a reference skeleton. If any mesh description
	 *  either doesn't contain a skin weight, or any of the skin weights reference a bone that doesn't exist on the
	 *  reference skeleton, this operation fails.
	 *  @param InSkeletalMesh The skeletal mesh to initialize. It has to be freshly created and be completely empty.
	 *  @param InMeshDescriptions The mesh description objects to create each subsequent LOD, starting from LOD0. 
	 *  @param InReferenceSkeleton The reference skeleton to use.
	 *  @param InParams The initialization parameters used.
	 */
	static SKELETALMESHUTILITIESCOMMON_API bool InitializeSkeletalMeshFromMeshDescriptions(
		USkeletalMesh* InSkeletalMesh,
		TArrayView<const FMeshDescription*> InMeshDescriptions,
		const FReferenceSkeleton& InReferenceSkeleton,
		const FInitializationParams& InParams = {}
		);
};

#endif
