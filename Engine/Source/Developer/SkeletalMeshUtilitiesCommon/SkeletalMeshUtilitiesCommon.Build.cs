// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    /**
     * Common algorithms and data structures used by MeshUtilities and other mesh-related modules.
     * This module must have minimal dependencies (we need Engine module and we should not depend on any editor module like UnrealEd) to allow it being used by various modules without introducing circular references.
     */
    public class SkeletalMeshUtilitiesCommon : ModuleRules
    {
        public SkeletalMeshUtilitiesCommon(ReadOnlyTargetRules Target) : base(Target)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
					"ClothingSystemRuntimeCommon",
					"Core",
					"CoreUObject",
					"Engine",
					"ImageCore",
					"InterchangeCore",
					"MeshDescription",
					"MeshUtilitiesCommon",
					"RenderCore",
					"SkeletalMeshDescription",
					"Slate",
					"StaticMeshDescription",
                }
            );
        }
    }
}
